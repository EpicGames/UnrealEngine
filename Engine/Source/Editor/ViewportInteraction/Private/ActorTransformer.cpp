// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorTransformer.h"
#include "ActorViewportTransformable.h"
#include "ViewportWorldInteraction.h"
#include "Engine/Selection.h"
#include "ViewportInteractionAssetContainer.h"
#include "ViewportInteractor.h"
#include "Editor.h"
#include "Editor/GroupActor.h"
#include "ActorGroupingUtils.h"
#include <utility>

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorTransformer)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


void UActorTransformer::Init( UViewportWorldInteraction* InitViewportWorldInteraction )
{
	Super::Init( InitViewportWorldInteraction );

	// Find out about selection changes
	USelection::SelectionChangedEvent.AddUObject( this, &UActorTransformer::OnActorSelectionChanged );
}


void UActorTransformer::Shutdown()
{
	USelection::SelectionChangedEvent.RemoveAll( this );

	Super::Shutdown();
}


void UActorTransformer::OnStartDragging(class UViewportInteractor* Interactor)
{
	const UViewportInteractionAssetContainer& AssetContainer = ViewportWorldInteraction->GetAssetContainer();
	const FVector& SoundLocation = Interactor->GetInteractorData().GizmoLastTransform.GetLocation();
	const EViewportInteractionDraggingMode DraggingMode = Interactor->GetDraggingMode();

	if (DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo)
	{
		ViewportWorldInteraction->PlaySound(AssetContainer.GizmoHandleSelectedSound, SoundLocation);
	}
	else if (DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
		DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact)
	{
		ViewportWorldInteraction->PlaySound(AssetContainer.SelectionStartDragSound, SoundLocation);
	}
}

void UActorTransformer::OnStopDragging(class UViewportInteractor* Interactor)
{
	const UViewportInteractionAssetContainer& AssetContainer = ViewportWorldInteraction->GetAssetContainer();
	const FVector& SoundLocation = Interactor->GetInteractorData().GizmoLastTransform.GetLocation();
	const EViewportInteractionDraggingMode DraggingMode = Interactor->GetDraggingMode();

	if (DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo)
	{
		ViewportWorldInteraction->PlaySound(AssetContainer.GizmoHandleDropSound, SoundLocation);
	}
	else if (DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
		DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact)
	{
		ViewportWorldInteraction->PlaySound(AssetContainer.SelectionDropSound, SoundLocation);
	}
}


namespace UE::ViewportInteraction::Private {
	template <typename InContainerType>
	bool ActorHasParentInSelection(const AActor* Actor, const InContainerType& SelectionSet)
	{
		check(Actor);
		for (const AActor* ParentActor = Actor->GetAttachParentActor(); ParentActor; ParentActor = ParentActor->GetAttachParentActor())
		{
			if (SelectionSet.Contains(ParentActor))
			{
				return true;
			}
		}
		return false;
	}
}


void UActorTransformer::OnActorSelectionChanged( UObject* ChangedObject )
{
	using namespace UE::ViewportInteraction::Private;

	TArray<TUniquePtr<FViewportTransformable>> NewTransformables;

	USelection* ActorSelectionSet = GEditor->GetSelectedActors();

	TArray<AActor*> SelectedActors;
	ActorSelectionSet->GetSelectedObjects(SelectedActors);

	// Imitate the logic of FActorElementLevelEditorSelectionCustomization::AppendNormalizedActors
	TArray<AActor*> NormalizedActors; // maintains order
	TSet<AActor*> NormalizedActorSet; // quick membership tests

	auto AddUniqueNormalizedActor = [&NormalizedActors, &NormalizedActorSet](AActor* InActor)
	{
		if (!NormalizedActorSet.Contains(InActor))
		{
			NormalizedActors.Add(InActor);
			NormalizedActorSet.Add(InActor);
		}
	};

	for (AActor* SelectedActor : SelectedActors)
	{
		// We only are able to move objects that have a root scene component
		if (!SelectedActor || !SelectedActor->GetRootComponent())
		{
			continue;
		}

		// Ensure that only parent-most actors are included
		if (ActorHasParentInSelection(SelectedActor, SelectedActors))
		{
			continue;
		}

		// Expand groups
		AGroupActor* ParentGroup = AGroupActor::GetRootForActor(SelectedActor, true, true);
		if (ParentGroup && UActorGroupingUtils::IsGroupingActive())
		{
			// Skip if the group is already in the normalized list, since this logic will have already run
			if (!NormalizedActorSet.Contains(ParentGroup))
			{
				ParentGroup->ForEachActorInGroup(
					[AddUniqueNormalizedActor, &SelectedActors = std::as_const(SelectedActors)]
					(AActor* InGroupedActor, AGroupActor* InGroupActor)
					{
						// Check that we've not got a parent attachment within the group/selection
						if (GroupActorHelpers::ActorHasParentInGroup(InGroupedActor, InGroupActor)
							|| ActorHasParentInSelection(InGroupedActor, SelectedActors))
						{
							return;
						}

						AddUniqueNormalizedActor(InGroupedActor);
					});
			}
		}

		AddUniqueNormalizedActor(SelectedActor);
	}

	for (AActor* SelectedActor : NormalizedActors)
	{
		FActorViewportTransformable* Transformable = new FActorViewportTransformable();

		NewTransformables.Add(TUniquePtr<FViewportTransformable>(Transformable));

		Transformable->ActorWeakPtr = SelectedActor;
		Transformable->StartTransform = SelectedActor->GetTransform();
		for (UViewportInteractor* Interactor : ViewportWorldInteraction->GetInteractors())
		{
			if (Interactor->CanCarry())
			{
				Transformable->bShouldBeCarried = true;
				break;
			}
		}
	}

	const bool bNewObjectsSelected = true;
	ViewportWorldInteraction->SetTransformables( MoveTemp( NewTransformables ), bNewObjectsSelected );
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
