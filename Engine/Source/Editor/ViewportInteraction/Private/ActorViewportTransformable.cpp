// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorViewportTransformable.h"
#include "GameFramework/Actor.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Components/PrimitiveComponent.h"
#include "LevelEditorViewport.h"


void FActorViewportTransformable::ApplyTransform( const FTransform& NewTransform, const bool bSweep )
{
	AActor* Actor = ActorWeakPtr.Get();
	if( Actor != nullptr )
	{
		const FTransform& ExistingTransform = Actor->GetActorTransform();
		if( !ExistingTransform.Equals( NewTransform, 0.0f ) )
		{
			// If we're moving a non-movable actor while in simulate mode, go ahead and make it movable.  We're only
			// editing the PIE copy of the actor here, so this won't affect the actual editor world.
			if( GEditor->bIsSimulatingInEditor && Actor->GetWorld()->IsPlayInEditor() )
			{
				if( Actor->GetRootComponent() != nullptr && Actor->GetRootComponent()->Mobility != EComponentMobility::Movable )
				{
					Actor->GetRootComponent()->SetMobility( EComponentMobility::Movable );
				}
			}

			const bool bOnlyTranslationChanged =
				ExistingTransform.GetRotation() == NewTransform.GetRotation() &&
				ExistingTransform.GetScale3D() == NewTransform.GetScale3D();
			GEditor->BroadcastBeginObjectMovement(*Actor);
			
			FProperty* TransformProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());
			if (!bOnlyTranslationChanged)
			{
				if (ExistingTransform.GetRotation() != NewTransform.GetRotation())
				{
					TransformProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName());
				}
				else if (ExistingTransform.GetScale3D() != NewTransform.GetScale3D())
				{
					TransformProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName());
				}
			}
			
			FEditPropertyChain PropertyChain;
			PropertyChain.AddHead(TransformProperty);
			FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(Actor, PropertyChain);


			Actor->SetActorTransform( NewTransform, bSweep );
			//GWarn->Logf( TEXT( "SMOOTH: Actor %s to %s" ), *Actor->GetName(), *Transformable.TargetTransform.ToString() );

			// @todo vreditor: InvalidateLightingCacheDetailed() causes static mesh components to re-create their physics state, 
			// cancelling all velocity on the rigid body.  So we currently avoid calling it for simulated actors.
			if( !IsPhysicallySimulated() && !GIsDemoMode)
			{
				Actor->InvalidateLightingCacheDetailed( bOnlyTranslationChanged );
			}



			FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);
			// Broadcast Post Edit change notification, we can't call PostEditChangeProperty directly on Actor or ActorComponent from here since it wasn't pair with a proper PreEditChange
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Actor, PropertyChangedEvent);
			const bool bFinished = false;	// @todo gizmo: PostEditChange never called; and bFinished=true never known!!
			Actor->PostEditMove(bFinished);
			GEditor->BroadcastEndObjectMovement(*Actor);
		}
	}
}


const FTransform FActorViewportTransformable::GetTransform() const
{
	AActor* Actor = ActorWeakPtr.Get();
	if( Actor != nullptr )
	{
		return Actor->GetTransform();
	}
	else
	{
		return FTransform::Identity;
	}
}


FBox FActorViewportTransformable::BuildBoundingBox( const FTransform& BoundingBoxToWorld ) const
{
	FBox BoundingBox(ForceInit);
	AActor* Actor = ActorWeakPtr.Get();

	if( Actor != nullptr )
	{
		const FTransform WorldToBoundingBox = BoundingBoxToWorld.Inverse();
		const FTransform& ActorToWorld = Actor->GetTransform();
		const FTransform ActorToBoundingBox = ActorToWorld * WorldToBoundingBox;

		const bool bIncludeNonCollidingComponents = false;	// @todo gizmo: Disabled this because it causes lights to have huge bounds
		const FBox ActorSpaceBoundingBox = Actor->CalculateComponentsBoundingBoxInLocalSpace( bIncludeNonCollidingComponents );

		BoundingBox = ActorSpaceBoundingBox.TransformBy( ActorToBoundingBox );
	}

	return BoundingBox;
}


bool FActorViewportTransformable::IsPhysicallySimulated() const
{
	bool bIsPhysicallySimulated = false;

	AActor* Actor = ActorWeakPtr.Get();
	if( Actor != nullptr )
	{
		UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>( Actor->GetRootComponent() );
		if( RootPrimitiveComponent != nullptr )
		{
			if( RootPrimitiveComponent->IsSimulatingPhysics() )
			{
				bIsPhysicallySimulated = true;
			}
		}
	}

	return bIsPhysicallySimulated;
}


bool FActorViewportTransformable::ShouldBeCarried() const
{
	return bShouldBeCarried;
}


void FActorViewportTransformable::SetLinearVelocity( const FVector& NewVelocity )
{
	AActor* Actor = ActorWeakPtr.Get();
	if( Actor != nullptr )
	{
		UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>( Actor->GetRootComponent() );
		if( RootPrimitiveComponent != nullptr )
		{
			RootPrimitiveComponent->SetAllPhysicsLinearVelocity( NewVelocity );
		}
	}
}


FVector FActorViewportTransformable::GetLinearVelocity() const
{
	const AActor* Actor = ActorWeakPtr.Get();
	if (Actor != nullptr)
	{
		return Actor->GetVelocity();
	}

	return FVector::ZeroVector;
}

void FActorViewportTransformable::UpdateIgnoredActorList( TArray<AActor*>& IgnoredActors )
{
	AActor* Actor = ActorWeakPtr.Get();
	if( Actor != nullptr )
	{
		IgnoredActors.Add( Actor );
	}
}
