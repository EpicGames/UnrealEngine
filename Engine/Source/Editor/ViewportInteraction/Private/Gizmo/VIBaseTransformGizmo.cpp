// Copyright Epic Games, Inc. All Rights Reserved.

#include "VIBaseTransformGizmo.h"
#include "Misc/App.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/Material.h"
#include "VIGizmoHandle.h"
#include "ViewportDragOperation.h"
#include "VIGizmoHandleMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VIBaseTransformGizmo)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd
{
	static FAutoConsoleVariable GizmoSelectionAnimationDuration( TEXT( "VI.GizmoSelectionAnimationDuration" ), 0.15f, TEXT( "How long to animate the gizmo after objects are selected" ) );
	static FAutoConsoleVariable GizmoSelectionAnimationCurvePower( TEXT( "VI.GizmoSelectionAnimationCurvePower" ), 2.0f, TEXT( "Controls the animation curve for the gizmo after objects are selected" ) );
	static FAutoConsoleVariable GizmoShowMeasurementText( TEXT( "VI.GizmoShowMeasurementText" ), 0, TEXT( "When enabled, gizmo measurements will always be visible.  Otherwise, only when hovering over a scale/stretch gizmo handle" ) );
}

ABaseTransformGizmo::ABaseTransformGizmo( ) :
	WorldInteraction( nullptr )
{
	// Create root default scene component
	{
		SceneComponent = CreateDefaultSubobject<USceneComponent>( TEXT( "SceneComponent" ) );
		check( SceneComponent != nullptr );

		RootComponent = SceneComponent;
	}
}

void ABaseTransformGizmo::OnNewObjectsSelected()
{
	SelectedAtTime = FTimespan::FromSeconds( FApp::GetCurrentTime() );
}

UViewportDragOperationComponent* ABaseTransformGizmo::GetInteractionType( UActorComponent* DraggedComponent, TOptional<FTransformGizmoHandlePlacement>& OutHandlePlacement )
{
	OutHandlePlacement.Reset();
	if ( DraggedComponent != nullptr )
	{
		UStaticMeshComponent* DraggedMesh = Cast<UStaticMeshComponent>( DraggedComponent );
		if ( DraggedMesh != nullptr )
		{
			for ( UGizmoHandleGroup* HandleGroup : AllHandleGroups )
			{
				if ( HandleGroup != nullptr )
				{
					int32 HandIndex = HandleGroup->GetDraggedHandleIndex( DraggedMesh );
					if ( HandIndex != INDEX_NONE )
					{
						OutHandlePlacement = HandleGroup->MakeHandlePlacementForIndex(HandIndex);
						return HandleGroup->GetDragOperationComponent();
					}
				}
			}
		}
	}

	OutHandlePlacement.Reset();
	return nullptr;
}

float ABaseTransformGizmo::GetAnimationAlpha()
{
	// Update animation
	float AnimationAlpha = 0.0f;
	{
		const FTimespan CurrentTime = FTimespan::FromSeconds( FApp::GetCurrentTime() );
		const double TimeSinceSelectionChange = (CurrentTime - SelectedAtTime).GetTotalSeconds();
		const double AnimLength = VREd::GizmoSelectionAnimationDuration->GetFloat();
		if ( TimeSinceSelectionChange < AnimLength )
		{
			AnimationAlpha = FMath::Max( 0.0f, float(TimeSinceSelectionChange / AnimLength) );
		}
		else
		{
			AnimationAlpha = 1.0f;
		}

		// Apply a bit of a curve to the animation
		AnimationAlpha = FMath::Pow( AnimationAlpha, VREd::GizmoSelectionAnimationCurvePower->GetFloat() );
	}

	return AnimationAlpha;
}

void ABaseTransformGizmo::SetOwnerWorldInteraction( class UViewportWorldInteraction* InWorldInteraction )
{
	WorldInteraction = InWorldInteraction;
}

class UViewportWorldInteraction* ABaseTransformGizmo::GetOwnerWorldInteraction() const
{
	return WorldInteraction;
}

EGizmoHandleTypes ABaseTransformGizmo::GetGizmoType() const
{
	return GizmoType;
}

void ABaseTransformGizmo::GetBoundingBoxEdge( const FBox& Box, const int32 AxisIndex, const int32 EdgeIndex, FVector& OutVertex0, FVector& OutVertex1 )
{
	check( AxisIndex >= 0 && AxisIndex < 3 );
	check( EdgeIndex >= 0 && EdgeIndex < 4 );

	const FVector BackBottomLeft( Box.Min.X, Box.Min.Y, Box.Min.Z );
	const FVector BackBottomRight( Box.Min.X, Box.Max.Y, Box.Min.Z );
	const FVector BackTopLeft( Box.Min.X, Box.Min.Y, Box.Max.Z );
	const FVector BackTopRight( Box.Min.X, Box.Max.Y, Box.Max.Z );

	const FVector FrontBottomLeft( Box.Max.X, Box.Min.Y, Box.Min.Z );
	const FVector FrontBottomRight( Box.Max.X, Box.Max.Y, Box.Min.Z );
	const FVector FrontTopLeft( Box.Max.X, Box.Min.Y, Box.Max.Z );
	const FVector FrontTopRight( Box.Max.X, Box.Max.Y, Box.Max.Z );

	switch (AxisIndex)
	{
		case 0:	// Front/back
		{
			switch ( EdgeIndex )
			{
			case 0:	// Bottom left
				OutVertex0 = BackBottomLeft;
				OutVertex1 = FrontBottomLeft;
				break;

			case 1:	// Top left
				OutVertex0 = BackTopLeft;
				OutVertex1 = FrontTopLeft;
				break;

			case 2:	// Top right
				OutVertex0 = BackTopRight;
				OutVertex1 = FrontTopRight;
				break;

			case 3:	// Bottom right
				OutVertex0 = BackBottomRight;
				OutVertex1 = FrontBottomRight;
				break;
			}
			break;
		}

		case 1:
		{
			switch ( EdgeIndex )
			{
			case 0:	// Back bottom
				OutVertex0 = BackBottomLeft;
				OutVertex1 = BackBottomRight;
				break;

			case 1:	// Back top
				OutVertex0 = BackTopLeft;
				OutVertex1 = BackTopRight;
				break;

			case 2:	// Front top
				OutVertex0 = FrontTopLeft;
				OutVertex1 = FrontTopRight;
				break;

			case 3:	// Front bottom
				OutVertex0 = FrontBottomLeft;
				OutVertex1 = FrontBottomRight;
				break;
			}
			break;
		}

		case 2:
		{
			switch ( EdgeIndex )
			{
			case 0:	// Back left
				OutVertex0 = BackBottomLeft;
				OutVertex1 = BackTopLeft;
				break;

			case 1:	// Back right
				OutVertex0 = BackBottomRight;
				OutVertex1 = BackTopRight;
				break;

			case 2:	// Front right
				OutVertex0 = FrontBottomRight;
				OutVertex1 = FrontTopRight;
				break;

			case 3:	// Front left
				OutVertex0 = FrontBottomLeft;
				OutVertex1 = FrontTopLeft;
				break;
			}
			break;
		}
	}
}




void ABaseTransformGizmo::UpdateHandleVisibility(const EGizmoHandleTypes InGizmoType, const ECoordSystem InGizmoCoordinateSpace, const bool bInAllHandlesVisible, UActorComponent* DraggingHandle)
{
	for ( UGizmoHandleGroup* HandleGroup : AllHandleGroups )
	{
		if ( HandleGroup != nullptr )
		{
			const bool bIsTypeSupported = ( InGizmoType == EGizmoHandleTypes::All && HandleGroup->GetShowOnUniversalGizmo() ) || HandleGroup->GetHandleType() == InGizmoType;
			const bool bSupportsCurrentCoordinateSpace = HandleGroup->SupportsWorldCoordinateSpace() || InGizmoCoordinateSpace != COORD_World;

			for ( FGizmoHandle& Handle : HandleGroup->GetHandles() )
			{
				if( Handle.HandleMesh != nullptr )
				{
					const bool bShowIt = ( bIsTypeSupported && bSupportsCurrentCoordinateSpace && bInAllHandlesVisible ) || ( DraggingHandle != nullptr && DraggingHandle == Handle.HandleMesh );

					Handle.HandleMesh->SetVisibility( bShowIt );

					// Never allow ray queries to impact hidden handles
					Handle.HandleMesh->SetCollisionEnabled( bShowIt ? ECollisionEnabled::QueryOnly : ECollisionEnabled::NoCollision );
				}
			}
		}
	}
}

bool ABaseTransformGizmo::GetShowMeasurementText() const
{
	return VREd::GizmoShowMeasurementText->GetInt() != 0;
}

void FTransformGizmoHandlePlacement::GetCenterHandleCountAndFacingAxisIndex( int32& OutCenterHandleCount, int32& OutFacingAxisIndex, int32& OutCenterAxisIndex ) const
{
	OutCenterHandleCount = 0;
	OutFacingAxisIndex = INDEX_NONE;
	OutCenterAxisIndex = INDEX_NONE;
	for ( int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex )
	{
		if ( Axes[AxisIndex] == ETransformGizmoHandleDirection::Center )
		{
			++OutCenterHandleCount;
			OutCenterAxisIndex = AxisIndex;
		}
		else
		{
			OutFacingAxisIndex = AxisIndex;
		}
	}

	if ( OutCenterHandleCount < 2 )
	{
		OutFacingAxisIndex = INDEX_NONE;
	}

	if ( OutCenterHandleCount != 1 )
	{
		OutCenterAxisIndex = INDEX_NONE;
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
