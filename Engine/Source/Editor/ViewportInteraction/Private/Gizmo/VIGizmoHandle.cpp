// Copyright Epic Games, Inc. All Rights Reserved.

#include "VIGizmoHandle.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "GameFramework/WorldSettings.h"
#include "VIBaseTransformGizmo.h"
#include "ViewportWorldInteraction.h"
#include "ViewportDragOperation.h"
#include "ViewportInteractionAssetContainer.h"
#include "VIGizmoHandleMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VIGizmoHandle)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UGizmoHandleGroup::UGizmoHandleGroup()
	: Super(),
	GizmoMaterial(nullptr),
	TranslucentGizmoMaterial(nullptr),
	Handles(),
	OwningTransformGizmoActor(nullptr),
	bShowOnUniversalGizmo(true)
{
	DragOperationComponent = CreateDefaultSubobject<UViewportDragOperationComponent>( TEXT( "DragOperation" ) );
}

FTransformGizmoHandlePlacement UGizmoHandleGroup::MakeHandlePlacementForIndex( const int32 HandleIndex ) const
{
	FTransformGizmoHandlePlacement HandlePlacement;
	HandlePlacement.Axes[0] = (ETransformGizmoHandleDirection)(HandleIndex / 9);
	HandlePlacement.Axes[1] = (ETransformGizmoHandleDirection)((HandleIndex % 9) / 3);
	HandlePlacement.Axes[2] = (ETransformGizmoHandleDirection)((HandleIndex % 9) % 3);
	//	GWarn->Logf( TEXT( "%i = HandlePlacment[ %i %i %i ]" ), HandleIndex, (int32)HandlePlacement.Axes[ 0 ], (int32)HandlePlacement.Axes[ 1 ], (int32)HandlePlacement.Axes[ 2 ] );
	return HandlePlacement;
}

int32 UGizmoHandleGroup::MakeHandleIndex( const FTransformGizmoHandlePlacement HandlePlacement ) const
{
	const int32 HandleIndex = (int32)HandlePlacement.Axes[0] * 9 + (int32)HandlePlacement.Axes[1] * 3 + (int32)HandlePlacement.Axes[2];
	//	GWarn->Logf( TEXT( "HandlePlacment[ %i %i %i ] = %i" ), (int32)HandlePlacement.Axes[ 0 ], (int32)HandlePlacement.Axes[ 1 ], (int32)HandlePlacement.Axes[ 2 ], HandleIndex );
	return HandleIndex;
}

FString UGizmoHandleGroup::MakeHandleName( const FTransformGizmoHandlePlacement HandlePlacement ) const
{
	FString HandleName;
	int32 CenteredAxisCount = 0;
	for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		const ETransformGizmoHandleDirection HandleDirection = HandlePlacement.Axes[AxisIndex];

		if (HandleDirection == ETransformGizmoHandleDirection::Center)
		{
			++CenteredAxisCount;
		}
		else
		{
			switch (AxisIndex)
			{
			case 0:
				HandleName += (HandleDirection == ETransformGizmoHandleDirection::Negative) ? TEXT( "Back" ) : TEXT( "Front" );
				break;

			case 1:
				HandleName += (HandleDirection == ETransformGizmoHandleDirection::Negative) ? TEXT( "Left" ) : TEXT( "Right" );
				break;

			case 2:
				HandleName += (HandleDirection == ETransformGizmoHandleDirection::Negative) ? TEXT( "Bottom" ) : TEXT( "Top" );
				break;
			}
		}
	}

	if (CenteredAxisCount == 2)
	{
		HandleName += TEXT( "Center" );
	}
	else if (CenteredAxisCount == 3)
	{
		HandleName = TEXT( "Origin" );
	}

	return HandleName;
}

FVector UGizmoHandleGroup::GetAxisVector( const int32 AxisIndex, const ETransformGizmoHandleDirection HandleDirection )
{
	FVector AxisVector = FVector::ZeroVector;

	if (HandleDirection == ETransformGizmoHandleDirection::Center)
	{
		AxisVector = FVector::ZeroVector;
	}
	else
	{
		const bool bIsFacingPositiveDirection = HandleDirection == ETransformGizmoHandleDirection::Positive;
		switch (AxisIndex)
		{
		case 0:
			AxisVector = (bIsFacingPositiveDirection ? FVector::ForwardVector : -FVector::ForwardVector);
			break;

		case 1:
			AxisVector = (bIsFacingPositiveDirection ? FVector::RightVector : -FVector::RightVector);
			break;

		case 2:
			AxisVector = (bIsFacingPositiveDirection ? FVector::UpVector : -FVector::UpVector);
			break;
		}
	}

	return AxisVector;
}

void UGizmoHandleGroup::UpdateGizmoHandleGroup(const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, 
	const TArray< UActorComponent* >& HoveringOverHandles, float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	UpdateHoverAnimation(DraggingHandle, HoveringOverHandles, GizmoHoverAnimationDuration, bOutIsHoveringOrDraggingThisHandleGroup);
}

int32 UGizmoHandleGroup::GetDraggedHandleIndex(class UStaticMeshComponent* DraggedMesh)
{
	for( int32 HandleIndex = 0; HandleIndex < Handles.Num(); ++HandleIndex )
	{
		if( Handles[ HandleIndex ].HandleMesh == DraggedMesh )
		{
			return HandleIndex;
		}
	}
	return INDEX_NONE;
}

void UGizmoHandleGroup::SetGizmoMaterial( UMaterialInterface* Material )
{
	GizmoMaterial = Material;
}

void UGizmoHandleGroup::SetTranslucentGizmoMaterial( UMaterialInterface* Material )
{
	TranslucentGizmoMaterial = Material;
}

TArray<FGizmoHandle>& UGizmoHandleGroup::GetHandles()
{
	return Handles;
}

EGizmoHandleTypes UGizmoHandleGroup::GetHandleType() const
{
	return EGizmoHandleTypes::All;
}

void UGizmoHandleGroup::SetShowOnUniversalGizmo( const bool bInShowOnUniversal )
{
	bShowOnUniversalGizmo = bInShowOnUniversal;
}

bool UGizmoHandleGroup::GetShowOnUniversalGizmo() const
{
	return bShowOnUniversalGizmo;
}

void UGizmoHandleGroup::SetOwningTransformGizmo(class ABaseTransformGizmo* TransformGizmo)
{
	OwningTransformGizmoActor = TransformGizmo;
}

void UGizmoHandleGroup::UpdateHandleColor( const int32 AxisIndex, FGizmoHandle& Handle, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles )
{
	UStaticMeshComponent* HandleMesh = Handle.HandleMesh;

	if ( !HandleMesh->GetMaterial( 0 )->IsA( UMaterialInstanceDynamic::StaticClass() ) )
	{
		if (!GizmoMaterial)
		{
			const UViewportInteractionAssetContainer* AssetContainer = UViewportWorldInteraction::LoadAssetContainer();
			GizmoMaterial = AssetContainer->TransformGizmoMaterial;
		}

		UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create( GizmoMaterial, this );
		HandleMesh->SetMaterial( 0, MID );
	}
	if ( !HandleMesh->GetMaterial( 1 )->IsA( UMaterialInstanceDynamic::StaticClass() ) )
	{
		if (!TranslucentGizmoMaterial)
		{
			const UViewportInteractionAssetContainer* AssetContainer = UViewportWorldInteraction::LoadAssetContainer();
			TranslucentGizmoMaterial = AssetContainer->TranslucentTransformGizmoMaterial;
		}

		UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create( TranslucentGizmoMaterial, this );
		HandleMesh->SetMaterial( 1, MID );
	}
	UMaterialInstanceDynamic* MID0 = CastChecked<UMaterialInstanceDynamic>( HandleMesh->GetMaterial( 0 ) );
	UMaterialInstanceDynamic* MID1 = CastChecked<UMaterialInstanceDynamic>( HandleMesh->GetMaterial( 1 ) );		

	ABaseTransformGizmo* GizmoActor = CastChecked<ABaseTransformGizmo>( GetOwner() );
	if (GizmoActor)
	{
		UViewportWorldInteraction* WorldInteraction = GizmoActor->GetOwnerWorldInteraction();
		if (WorldInteraction)
		{
			FLinearColor HandleColor = WorldInteraction->GetColor(UViewportWorldInteraction::EColors::DefaultColor);
			if (HandleMesh == DraggingHandle)
			{
				HandleColor = WorldInteraction->GetColor(UViewportWorldInteraction::EColors::GizmoDragging);
			}
			else if (AxisIndex != INDEX_NONE)
			{
				switch (AxisIndex)
				{
				case 0:
					HandleColor = WorldInteraction->GetColor(UViewportWorldInteraction::EColors::Forward);
					break;

				case 1:
					HandleColor = WorldInteraction->GetColor(UViewportWorldInteraction::EColors::Right);
					break;

				case 2:
					HandleColor = WorldInteraction->GetColor(UViewportWorldInteraction::EColors::Up);
					break;
				}

				if (HoveringOverHandles.Contains(HandleMesh))
				{
					HandleColor = FLinearColor::LerpUsingHSV(HandleColor, WorldInteraction->GetColor(UViewportWorldInteraction::EColors::GizmoHover), Handle.HoverAlpha);
				}
			}

			static FName StaticHandleColorParameter("Color");
			MID0->SetVectorParameterValue(StaticHandleColorParameter, HandleColor);
			MID1->SetVectorParameterValue(StaticHandleColorParameter, HandleColor);
		}
	}
}

void UGizmoHandleGroup::UpdateVisibilityAndCollision(const EGizmoHandleTypes GizmoType, const ECoordSystem GizmoCoordinateSpace, const bool bAllHandlesVisible, const bool bAllowRotationAndScaleHandles, UActorComponent* DraggingHandle)
{
	const bool bIsTypeSupported = 
		( (GizmoType == EGizmoHandleTypes::All && GetShowOnUniversalGizmo()) || GetHandleType() == GizmoType ) &&
		( bAllowRotationAndScaleHandles || ( GetHandleType() != EGizmoHandleTypes::Rotate && GetHandleType() != EGizmoHandleTypes::Scale ) );

	const bool bSupportsCurrentCoordinateSpace = SupportsWorldCoordinateSpace() || GizmoCoordinateSpace != COORD_World;

	for (FGizmoHandle& Handle : GetHandles())
	{
		if (Handle.HandleMesh != nullptr)
		{
			const bool bShowIt = (bIsTypeSupported && bSupportsCurrentCoordinateSpace && bAllHandlesVisible) || (DraggingHandle != nullptr && DraggingHandle == Handle.HandleMesh);

			Handle.HandleMesh->SetVisibility(bShowIt);

			// Never allow ray queries to impact hidden handles
			Handle.HandleMesh->SetCollisionEnabled(bShowIt ? ECollisionEnabled::QueryOnly : ECollisionEnabled::NoCollision);
		}
	}
}

UViewportDragOperationComponent* UGizmoHandleGroup::GetDragOperationComponent()
{
	return DragOperationComponent;
}

class UGizmoHandleMeshComponent* UGizmoHandleGroup::CreateMeshHandle( class UStaticMesh* HandleMesh, const FString& ComponentName )
{
	const bool bAllowGizmoLighting = false;	// @todo vreditor: Not sure if we want this for gizmos or not yet.  Needs feedback.  Also they're translucent right now.

	UGizmoHandleMeshComponent* HandleComponent = CreateDefaultSubobject<UGizmoHandleMeshComponent>( *ComponentName );
	check( HandleComponent != nullptr );

	HandleComponent->SetCanEverAffectNavigation( false );
	HandleComponent->SetStaticMesh( HandleMesh );
	HandleComponent->SetMobility( EComponentMobility::Movable );
	HandleComponent->SetupAttachment( this );

	HandleComponent->SetCollisionEnabled( ECollisionEnabled::QueryOnly );
	HandleComponent->SetCollisionResponseToAllChannels( ECR_Ignore );
	HandleComponent->SetCollisionResponseToChannel( COLLISION_GIZMO, ECollisionResponse::ECR_Block );
	HandleComponent->SetCollisionObjectType( COLLISION_GIZMO );

	HandleComponent->SetGenerateOverlapEvents(false);
	HandleComponent->bCastDynamicShadow = bAllowGizmoLighting;
	HandleComponent->bCastStaticShadow = false;
	HandleComponent->bAffectDistanceFieldLighting = bAllowGizmoLighting;
	HandleComponent->bAffectDynamicIndirectLighting = bAllowGizmoLighting;
	//HandleComponent->bUseEditorCompositing = true;

	return HandleComponent;
}

UGizmoHandleMeshComponent* UGizmoHandleGroup::CreateAndAddMeshHandle( UStaticMesh* HandleMesh, const FString& ComponentName, const FTransformGizmoHandlePlacement& HandlePlacement )
{
	UGizmoHandleMeshComponent* HandleComponent = CreateMeshHandle( HandleMesh, ComponentName );
	AddMeshToHandles( HandleComponent, HandlePlacement );
	return HandleComponent;
}

void UGizmoHandleGroup::AddMeshToHandles( UGizmoHandleMeshComponent* HandleMeshComponent, const FTransformGizmoHandlePlacement& HandlePlacement )
{
	int32 HandleIndex = MakeHandleIndex( HandlePlacement );
	if (Handles.Num() < (HandleIndex + 1))
	{
		Handles.SetNumZeroed( HandleIndex + 1 );
	}
	Handles[HandleIndex].HandleMesh = HandleMeshComponent;
}

FTransformGizmoHandlePlacement UGizmoHandleGroup::GetHandlePlacement( const int32 X, const int32 Y, const int32 Z ) const
{
	FTransformGizmoHandlePlacement HandlePlacement;
	HandlePlacement.Axes[0] = (ETransformGizmoHandleDirection)X;
	HandlePlacement.Axes[1] = (ETransformGizmoHandleDirection)Y;
	HandlePlacement.Axes[2] = (ETransformGizmoHandleDirection)Z;

	return HandlePlacement;
}

void UGizmoHandleGroup::UpdateHoverAnimation( UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	for (FGizmoHandle& Handle : Handles)
	{
		const bool bIsHoveringOverHandle = HoveringOverHandles.Contains( Handle.HandleMesh ) || (DraggingHandle != nullptr && DraggingHandle == Handle.HandleMesh);

		if (bIsHoveringOverHandle)
		{
			Handle.HoverAlpha += GetWorld()->GetDeltaSeconds() / GizmoHoverAnimationDuration;
			bOutIsHoveringOrDraggingThisHandleGroup = true;
		}
		else
		{
			Handle.HoverAlpha -= GetWorld()->GetDeltaSeconds() / GizmoHoverAnimationDuration;
		}
		Handle.HoverAlpha = FMath::Clamp( Handle.HoverAlpha, 0.0f, 1.0f );
	}
}

void UAxisGizmoHandleGroup::CreateHandles( UStaticMesh* HandleMesh, const FString& HandleComponentName )
{
	for ( int32 X = 0; X < 3; ++X )
	{
		for ( int32 Y = 0; Y < 3; ++Y )
		{
			for ( int32 Z = 0; Z < 3; ++Z )
			{
				FTransformGizmoHandlePlacement HandlePlacement = GetHandlePlacement(X, Y, Z);

				int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
				HandlePlacement.GetCenterHandleCountAndFacingAxisIndex( /* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex);

				// Don't allow translation/stretching/rotation from the origin
				if ( CenterHandleCount < 3 )
				{
					const FString HandleName = MakeHandleName(HandlePlacement);

					// Only happens on the center of an axis. And we only bother drawing one for the "positive" direction.
					if ( CenterHandleCount == 2 && HandlePlacement.Axes[FacingAxisIndex] == ETransformGizmoHandleDirection::Positive )
					{
						// Plane translation handle
						FString ComponentName = HandleName + HandleName;
						CreateAndAddMeshHandle( HandleMesh, ComponentName, HandlePlacement );
					}
				}
			}
		}
	}
}

void UAxisGizmoHandleGroup::UpdateHandlesRelativeTransformOnAxis( const FTransform& HandleToCenter, const float AnimationAlpha, const float GizmoScale, const float GizmoHoverScale,
	const FVector& ViewLocation, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles )
{
	const float WorldScaleFactor = GetWorld()->GetWorldSettings()->WorldToMeters / 100.0f;

	for (int32 HandleIndex = 0; HandleIndex < Handles.Num(); ++HandleIndex)
	{
		FGizmoHandle& Handle = Handles[HandleIndex];
		UStaticMeshComponent* GizmoHandleMeshComponent = Handle.HandleMesh;
		if (GizmoHandleMeshComponent != nullptr)	// Can be null if no handle for this specific placement
		{
			const FTransformGizmoHandlePlacement HandlePlacement = MakeHandlePlacementForIndex( HandleIndex );

			int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
			HandlePlacement.GetCenterHandleCountAndFacingAxisIndex( /* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex);
			
			if (DraggingHandle == nullptr)
			{
				FVector GizmoSpaceFacingAxisVector = GetAxisVector( FacingAxisIndex, HandlePlacement.Axes[FacingAxisIndex] );
			
				// Check on which side we are relative to the gizmo
				const FVector GizmoSpaceViewLocation = GetOwner()->GetTransform().InverseTransformPosition( ViewLocation );
				if ( GizmoSpaceViewLocation[ FacingAxisIndex ] < 0 )
				{
					GizmoSpaceFacingAxisVector[ FacingAxisIndex ] *= -1.0f;
				}

				const FTransform GizmoOriginToFacingAxisRotation( GizmoSpaceFacingAxisVector.ToOrientationQuat() );
	
				FTransform HandleToGizmoOrigin = HandleToCenter * GizmoOriginToFacingAxisRotation;
				
				// Check on the axis if the offset is on the other side of the object compared to the view. Switch the offset to the side of the view if it does
				FVector GizmoSpaceFacingAxisOffset = HandleToGizmoOrigin.GetLocation();
				for ( int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex )
				{		
					if ( AxisIndex != FacingAxisIndex &&
						( ( GizmoSpaceFacingAxisOffset[ AxisIndex ] > 0 && GizmoSpaceViewLocation[ AxisIndex ] < 0 ) ||
						  ( GizmoSpaceFacingAxisOffset[ AxisIndex ] < 0 && GizmoSpaceViewLocation[ AxisIndex ] > 0 ) ) )
					{
						GizmoSpaceFacingAxisOffset[ AxisIndex ] *= -1.0f;
					}	
				}

				GizmoSpaceFacingAxisOffset *= AnimationAlpha;

				HandleToGizmoOrigin.SetLocation( GizmoSpaceFacingAxisOffset * GizmoScale );

				// Set the final transform
				GizmoHandleMeshComponent->SetRelativeTransform( HandleToGizmoOrigin );
			
				float GizmoHandleScale = GizmoScale;

				// Make the handle bigger while hovered (but don't affect the offset -- we want it to scale about it's origin)
				GizmoHandleScale *= FMath::Lerp( 1.0f, GizmoHoverScale, Handle.HoverAlpha );
			
				GizmoHandleScale *= AnimationAlpha;
				GizmoHandleMeshComponent->SetRelativeScale3D( FVector( GizmoHandleScale ) );
			}

			// Update material
			UpdateHandleColor( FacingAxisIndex, Handle, DraggingHandle, HoveringOverHandles );
		}
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
