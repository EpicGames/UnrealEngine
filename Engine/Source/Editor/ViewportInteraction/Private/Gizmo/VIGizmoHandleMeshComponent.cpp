// Copyright Epic Games, Inc. All Rights Reserved.

#include "VIGizmoHandleMeshComponent.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"

#include "RHIResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VIGizmoHandleMeshComponent)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class FGimzoHandleSceneProxy final : public FStaticMeshSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FGimzoHandleSceneProxy(UGizmoHandleMeshComponent* InComponent)
		: FStaticMeshSceneProxy(InComponent, false)
	{
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FSceneView* View = Views[ViewIndex];

			if (IsShown(View) && (VisibilityMap & (1 << ViewIndex)))
			{
				FLODMask LODMask = GetLODMask(View);

				for (int32 LODIndex = 0; LODIndex < RenderData->LODResources.Num(); LODIndex++)
				{
					if (LODMask.ContainsLOD(LODIndex))
					{
						const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];

						// Draw the static mesh sections.
						for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
						{
							for (int32 BatchIndex = 0; BatchIndex < GetNumMeshBatches(); BatchIndex++)
							{
								const bool bSectionIsSelected = false;
								const bool bIsHovered = false;
								FMeshBatch& MeshElement = Collector.AllocateMesh();

								if (GetMeshElement(LODIndex, BatchIndex, SectionIndex, SDPG_World, bSectionIsSelected, true, MeshElement))
								{
									Collector.AddMesh(ViewIndex, MeshElement);
								}
							}
						}
					}
				}
			}
		}
	}


	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result = FStaticMeshSceneProxy::GetViewRelevance(View);
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = false;
		Result.bEditorNoDepthTestPrimitiveRelevance = true;
		return Result;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return sizeof *this + GetAllocatedSize(); }
	SIZE_T GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }
};

UGizmoHandleMeshComponent::UGizmoHandleMeshComponent() :
	Super()
{
}

FPrimitiveSceneProxy* UGizmoHandleMeshComponent::CreateSceneProxy()
{
	// If a static mesh does not exist then this component cannot be added to the scene.
	if (GetStaticMesh() == nullptr)
	{
		return Super::CreateSceneProxy();
	}

	return new FGimzoHandleSceneProxy(this);
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
