// Copyright Epic Games, Inc. All Rights Reserved.

#include "Gizmo/VIPivotTransformGizmo.h"
#include "Engine/World.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/WorldSettings.h"
#include "VIStretchGizmoHandle.h"
#include "VIUniformScaleGizmoHandle.h"
#include "ViewportWorldInteraction.h"
#include "ViewportInteractor.h"
#include "ViewportInteractionAssetContainer.h"
#include "Materials/Material.h"
#include "Engine/Font.h"
#include "Engine/CollisionProfile.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/TextRenderComponent.h"
#include "VIGizmoHandleMeshComponent.h"
#include "Math/UnitConversion.h"
#include "ViewportInteractionDragOperations.h"
#include "UObject/StrongObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VIPivotTransformGizmo)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd //@todo VREditor: Duplicates of TransformGizmo
{
	// @todo vreditor tweak: Tweak out console variables
	static FAutoConsoleVariable PivotGizmoMinDistanceForScaling( TEXT( "VI.PivotGizmoMinDistanceForScaling" ), 0.0f, TEXT( "How far away the camera needs to be from an object before we'll start scaling it based on distance"));
	static FAutoConsoleVariable PivotGizmoDistanceScaleFactor( TEXT( "VI.PivotGizmoDistanceScaleFactor" ), 0.0035f, TEXT( "How much the gizmo handles should increase in size with distance from the camera, to make it easier to select" ) );
	static FAutoConsoleVariable PivotGizmoTranslationPivotOffsetX( TEXT("VI.PivotGizmoTranslationPivotOffsetX" ), 30.0f, TEXT( "How much the translation cylinder is offsetted from the pivot" ) );
	static FAutoConsoleVariable PivotGizmoScalePivotOffsetX( TEXT( "VI.PivotGizmoScalePivotOffsetX" ), 120.0f, TEXT( "How much the non-uniform scale is offsetted from the pivot" ) );
	static FAutoConsoleVariable PivotGizmoPlaneTranslationPivotOffsetYZ(TEXT("VI.PivotGizmoPlaneTranslationPivotOffsetYZ" ), 40.0f, TEXT( "How much the plane translation is offsetted from the pivot" ) );
	static FAutoConsoleVariable PivotGizmoTranslationScaleMultiply( TEXT( "VI.PivotGizmoTranslationScaleMultiply" ), 2.0f, TEXT( "Multiplies translation handles scale" ) );
	static FAutoConsoleVariable PivotGizmoTranslationHoverScaleMultiply( TEXT( "VI.PivotGizmoTranslationHoverScaleMultiply" ), 0.75f, TEXT( "Multiplies translation handles hover scale" ) );
	static FAutoConsoleVariable PivotGizmoAimAtShrinkSize( TEXT( "VI.PivotGizmoAimAtShrinkSize" ), 0.3f, TEXT( "The minimum size when not aiming at the gizmo (0 to 1)" ) );
	static FAutoConsoleVariable PivotGizmoAimAtAnimationSpeed( TEXT( "VI.PivotGizmoAimAtAnimationSpeed" ), 0.15f, TEXT( "The speed to animate to the gizmo full size when aiming at it" ) );
}

APivotTransformGizmo::APivotTransformGizmo() :
	Super(),
	AimingAtGizmoScaleAlpha(0.0f),
	LastDraggingHandle(nullptr)
{
	bIsEditorOnlyActor = true;

	UniformScaleGizmoHandleGroup = CreateDefaultSubobject<UUniformScaleGizmoHandleGroup>( TEXT( "UniformScaleHandles" ), true );
	UniformScaleGizmoHandleGroup->SetOwningTransformGizmo(this);
	UniformScaleGizmoHandleGroup->SetupAttachment( SceneComponent );
	AllHandleGroups.Add( UniformScaleGizmoHandleGroup );

	TranslationGizmoHandleGroup = CreateDefaultSubobject<UPivotTranslationGizmoHandleGroup>(TEXT("TranslationHandles"), true);
	TranslationGizmoHandleGroup->SetOwningTransformGizmo(this);
	TranslationGizmoHandleGroup->SetupAttachment( SceneComponent );
	AllHandleGroups.Add(TranslationGizmoHandleGroup);

	ScaleGizmoHandleGroup = CreateDefaultSubobject<UPivotScaleGizmoHandleGroup>( TEXT( "ScaleHandles" ), true );
	ScaleGizmoHandleGroup->SetOwningTransformGizmo(this);
	ScaleGizmoHandleGroup->SetupAttachment( SceneComponent );
	AllHandleGroups.Add( ScaleGizmoHandleGroup );

	PlaneTranslationGizmoHandleGroup = CreateDefaultSubobject<UPivotPlaneTranslationGizmoHandleGroup>( TEXT( "PlaneTranslationHandles" ), true );
	PlaneTranslationGizmoHandleGroup->SetOwningTransformGizmo(this);
	PlaneTranslationGizmoHandleGroup->SetupAttachment( SceneComponent );
	AllHandleGroups.Add( PlaneTranslationGizmoHandleGroup );

	RotationGizmoHandleGroup = CreateDefaultSubobject<UPivotRotationGizmoHandleGroup>( TEXT( "RotationHandles" ), true );
	RotationGizmoHandleGroup->SetOwningTransformGizmo(this);
	RotationGizmoHandleGroup->SetupAttachment( SceneComponent );
	AllHandleGroups.Add( RotationGizmoHandleGroup );

	StretchGizmoHandleGroup = CreateDefaultSubobject<UStretchGizmoHandleGroup>( TEXT( "StretchHandles" ), true );
	StretchGizmoHandleGroup->SetOwningTransformGizmo(this);
	StretchGizmoHandleGroup->SetShowOnUniversalGizmo( false );
	StretchGizmoHandleGroup->SetupAttachment( SceneComponent );
	AllHandleGroups.Add( StretchGizmoHandleGroup );

	// There may already be some objects selected as we switch into VR mode, so we'll pretend that just happened so
	// that we can make sure all transitions complete properly
	OnNewObjectsSelected();
}

void APivotTransformGizmo::UpdateGizmo(const EGizmoHandleTypes InGizmoType, const ECoordSystem InGizmoCoordinateSpace, const FTransform& InLocalToWorld, const FBox& InLocalBounds, const FVector& InViewLocation, const float InScaleMultiplier, bool bInAllHandlesVisible, const bool bInAllowRotationAndScaleHandles, class UActorComponent* DraggingHandle, const TArray<UActorComponent*>& InHoveringOverHandles, const float InGizmoHoverScale, const float InGizmoHoverAnimationDuration)
{
	GizmoType = InGizmoType;

	// Position the gizmo at the location of the first selected actor
	SetActorTransform(InLocalToWorld);

	// Increase scale with distance, to make gizmo handles easier to click on
	const double WorldSpaceDistanceToToPivot = FMath::Max(VREd::PivotGizmoMinDistanceForScaling->GetFloat(), FMath::Sqrt(FVector::DistSquared( GetActorLocation(), InViewLocation )));
	const double WorldScaleFactor = WorldInteraction->GetWorldScaleFactor();
	const double GizmoScale = (InScaleMultiplier * ((WorldSpaceDistanceToToPivot / WorldScaleFactor) * VREd::PivotGizmoDistanceScaleFactor->GetFloat())) * WorldScaleFactor;
	const bool bIsWorldSpaceGizmo = (WorldInteraction->GetTransformGizmoCoordinateSpace() == COORD_World);

	if (LastDraggingHandle != nullptr && DraggingHandle == nullptr)
	{
		AimingAtGizmoScaleAlpha = 0.0f;
	}

	double AnimatedGizmoScale = GizmoScale;
	// Only scale the gizmo down when not aiming at it for VR implementations
	if (WorldInteraction->IsInVR())
	{
		bool bIsAimingTowards = false;
		const double GizmoRadius = (GizmoScale * 350) * 0.5f;

		// Check if any interactor has a laser close to the gizmo
		for (UViewportInteractor* Interactor : WorldInteraction->GetInteractors())
		{	
			// We only want the interactor to affect the size when aiming at the gizmo if they are dragging nothing
			if (!Interactor->GetIsLaserBlocked() && Interactor->GetDraggingMode() != EViewportInteractionDraggingMode::World)
			{
				FVector LaserPointerStart, LaserPointerEnd;
				if (Interactor->GetLaserPointer(/*Out*/ LaserPointerStart, /*Out*/ LaserPointerEnd))
				{
					const FVector ClosestPointOnLaser = FMath::ClosestPointOnLine(LaserPointerStart, LaserPointerEnd, InLocalToWorld.GetLocation());
					const double ClosestPointDistance = (ClosestPointOnLaser - InLocalToWorld.GetLocation()).Size();
					if (ClosestPointDistance <= GizmoRadius)
					{
						bIsAimingTowards = true;
						break;
					}
				}
			}
		}

		const float DeltaTime = WorldInteraction->GetCurrentDeltaTime();
		if (bIsAimingTowards)
		{
			AimingAtGizmoScaleAlpha += DeltaTime / VREd::PivotGizmoAimAtAnimationSpeed->GetFloat();
		}
		else
		{
			AimingAtGizmoScaleAlpha -= DeltaTime / VREd::PivotGizmoAimAtAnimationSpeed->GetFloat();
		}
		AimingAtGizmoScaleAlpha = FMath::Clamp(AimingAtGizmoScaleAlpha, VREd::PivotGizmoAimAtShrinkSize->GetFloat(), 1.0f);

		AnimatedGizmoScale *= AimingAtGizmoScaleAlpha;
	}

	// Update animation
	float AnimationAlpha = GetAnimationAlpha();

	// Update all the handles
	for (UGizmoHandleGroup* HandleGroup : AllHandleGroups)
	{
		if (HandleGroup != nullptr)
		{
			bool bIsHoveringOrDraggingThisHandleGroup = false;
			
			const double Scale = HandleGroup == StretchGizmoHandleGroup ? GizmoScale : AnimatedGizmoScale;
			const float FloatScale = FloatCastChecked<float>(Scale, 1.0 / 16.0);

			HandleGroup->UpdateGizmoHandleGroup(InLocalToWorld, InLocalBounds, InViewLocation, bInAllHandlesVisible, DraggingHandle,
				InHoveringOverHandles, AnimationAlpha, FloatScale, InGizmoHoverScale, InGizmoHoverAnimationDuration, /* Out */ bIsHoveringOrDraggingThisHandleGroup);

			
			if (HandleGroup != RotationGizmoHandleGroup)
			{
				HandleGroup->UpdateVisibilityAndCollision(InGizmoType, InGizmoCoordinateSpace, bInAllHandlesVisible, bInAllowRotationAndScaleHandles, DraggingHandle);
			}
		}
	}

	LastDraggingHandle = DraggingHandle;
}

/************************************************************************/
/* Translation                                                          */
/************************************************************************/
UPivotTranslationGizmoHandleGroup::UPivotTranslationGizmoHandleGroup() :
	Super()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TStrongObjectPtr<const UViewportInteractionAssetContainer> AssetContainer(UViewportWorldInteraction::LoadAssetContainer());
	CreateHandles( AssetContainer->TranslationHandleMesh, FString( "PivotTranslationHandle" ) );

	DragOperationComponent->SetDragOperationClass(UTranslationDragOperation::StaticClass());
}


void UPivotTranslationGizmoHandleGroup::UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, 
	const TArray< UActorComponent* >& HoveringOverHandles, float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	// Call parent implementation (updates hover animation)
	Super::UpdateGizmoHandleGroup( LocalToWorld, LocalBounds, ViewLocation, bAllHandlesVisible, DraggingHandle, HoveringOverHandles, AnimationAlpha,
		GizmoScale, GizmoHoverScale, GizmoHoverAnimationDuration, bOutIsHoveringOrDraggingThisHandleGroup );
	
	const float MultipliedGizmoScale = GizmoScale * VREd::PivotGizmoTranslationScaleMultiply->GetFloat();
	const float MultipliedGizmoHoverScale = GizmoHoverScale * VREd::PivotGizmoTranslationHoverScaleMultiply->GetFloat();

	UpdateHandlesRelativeTransformOnAxis( FTransform( FVector( VREd::PivotGizmoTranslationPivotOffsetX->GetFloat(), 0, 0 ) ), AnimationAlpha, MultipliedGizmoScale, MultipliedGizmoHoverScale, ViewLocation, DraggingHandle, HoveringOverHandles );
}

EGizmoHandleTypes UPivotTranslationGizmoHandleGroup::GetHandleType() const
{
	return EGizmoHandleTypes::Translate;
}

/************************************************************************/
/* Scale	                                                            */
/************************************************************************/
UPivotScaleGizmoHandleGroup::UPivotScaleGizmoHandleGroup() :
	Super()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TStrongObjectPtr<const UViewportInteractionAssetContainer> AssetContainer(UViewportWorldInteraction::LoadAssetContainer());
	CreateHandles( AssetContainer->UniformScaleHandleMesh, FString( "PivotScaleHandle" ) );	

	DragOperationComponent->SetDragOperationClass(UScaleDragOperation::StaticClass());
}

void UPivotScaleGizmoHandleGroup::UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, 
	const TArray< UActorComponent* >& HoveringOverHandles, float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	// Call parent implementation (updates hover animation)
	Super::UpdateGizmoHandleGroup( LocalToWorld, LocalBounds, ViewLocation,bAllHandlesVisible, DraggingHandle, HoveringOverHandles, AnimationAlpha, 
		GizmoScale, GizmoHoverScale, GizmoHoverAnimationDuration, bOutIsHoveringOrDraggingThisHandleGroup );

	UpdateHandlesRelativeTransformOnAxis( FTransform( FVector( VREd::PivotGizmoScalePivotOffsetX->GetFloat(), 0, 0 ) ), AnimationAlpha, GizmoScale, GizmoHoverScale, ViewLocation, DraggingHandle, HoveringOverHandles );
}

EGizmoHandleTypes UPivotScaleGizmoHandleGroup::GetHandleType() const
{
	return EGizmoHandleTypes::Scale;
}

bool UPivotScaleGizmoHandleGroup::SupportsWorldCoordinateSpace() const
{
	return false;
}

/************************************************************************/
/* Plane Translation	                                                */
/************************************************************************/
UPivotPlaneTranslationGizmoHandleGroup::UPivotPlaneTranslationGizmoHandleGroup() :
	Super()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TStrongObjectPtr<const UViewportInteractionAssetContainer> AssetContainer(UViewportWorldInteraction::LoadAssetContainer());
	CreateHandles( AssetContainer->PlaneTranslationHandleMesh, FString( "PlaneTranslationHandle" ) );

	DragOperationComponent->SetDragOperationClass(UPlaneTranslationDragOperation::StaticClass());
}

void UPivotPlaneTranslationGizmoHandleGroup::UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle,
	const TArray< UActorComponent* >& HoveringOverHandles, float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	// Call parent implementation (updates hover animation)
	Super::UpdateGizmoHandleGroup( LocalToWorld, LocalBounds, ViewLocation, bAllHandlesVisible, DraggingHandle, HoveringOverHandles, AnimationAlpha,
		GizmoScale, GizmoHoverScale, GizmoHoverAnimationDuration, bOutIsHoveringOrDraggingThisHandleGroup );

	UpdateHandlesRelativeTransformOnAxis( FTransform( FVector( 0, VREd::PivotGizmoPlaneTranslationPivotOffsetYZ->GetFloat(), VREd::PivotGizmoPlaneTranslationPivotOffsetYZ->GetFloat() ) ), 
		AnimationAlpha, GizmoScale, GizmoHoverScale, ViewLocation, DraggingHandle, HoveringOverHandles );
}

EGizmoHandleTypes UPivotPlaneTranslationGizmoHandleGroup::GetHandleType() const
{
	return EGizmoHandleTypes::Translate;
}

/************************************************************************/
/* Rotation																*/
/************************************************************************/
UPivotRotationGizmoHandleGroup::UPivotRotationGizmoHandleGroup() :
	Super()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TStrongObjectPtr<const UViewportInteractionAssetContainer> AssetContainer(UViewportWorldInteraction::LoadAssetContainer());

	UStaticMesh* QuarterRotationHandleMesh = AssetContainer->RotationHandleMesh;
	CreateHandles(QuarterRotationHandleMesh, FString("RotationHandle"));

	{
		RootFullRotationHandleComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootFullRotationHandleComponent"));
		RootFullRotationHandleComponent->SetMobility(EComponentMobility::Movable);
		RootFullRotationHandleComponent->SetupAttachment(this);
	
		UStaticMesh* FullRotationHandleMesh = AssetContainer->RotationHandleSelectedMesh;
		check(FullRotationHandleMesh != nullptr);

		FullRotationHandleMeshComponent = CreateMeshHandle(FullRotationHandleMesh, FString("FullRotationHandle"));
		FullRotationHandleMeshComponent->SetVisibility(false);
		FullRotationHandleMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		FullRotationHandleMeshComponent->SetupAttachment(RootFullRotationHandleComponent);
	}

	{
		UStaticMesh* RotationHandleIndicatorMesh = AssetContainer->StartRotationIndicatorMesh;
		check(RotationHandleIndicatorMesh != nullptr);

		//Start rotation indicator
		RootStartRotationIdicatorComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootStartRotationIndicator"));
		StartRotationIndicatorMeshComponent = CreateDefaultSubobject<UGizmoHandleMeshComponent>(TEXT("StartRotationIndicator"));
		SetupIndicator(RootStartRotationIdicatorComponent, StartRotationIndicatorMeshComponent, RotationHandleIndicatorMesh);
	}

	{
		UStaticMesh* RotationHandleIndicatorMesh = AssetContainer->CurrentRotationIndicatorMesh;
		check(RotationHandleIndicatorMesh != nullptr);

		//Delta rotation indicator
		RootDeltaRotationIndicatorComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootDeltaRotationIndicator"));
		DeltaRotationIndicatorMeshComponent = CreateDefaultSubobject<UGizmoHandleMeshComponent>(TEXT("DeltaRotationIndicator"));
		SetupIndicator(RootDeltaRotationIndicatorComponent, DeltaRotationIndicatorMeshComponent, RotationHandleIndicatorMesh);
	}

	{
		UMaterialInstanceDynamic* DynamicMaterialInst = UMaterialInstanceDynamic::Create(AssetContainer->TransformGizmoMaterial, GetTransientPackage());
		check(DynamicMaterialInst != nullptr);

		DeltaRotationIndicatorMeshComponent->SetMaterial(0, DynamicMaterialInst);
		StartRotationIndicatorMeshComponent->SetMaterial(0, DynamicMaterialInst);
		FullRotationHandleMeshComponent->SetMaterial(0, DynamicMaterialInst);

		UMaterialInstanceDynamic* TranslucentDynamicMaterialInst = UMaterialInstanceDynamic::Create(AssetContainer->TranslucentTransformGizmoMaterial, GetTransientPackage());
		check(TranslucentDynamicMaterialInst != nullptr);

		DeltaRotationIndicatorMeshComponent->SetMaterial(1, TranslucentDynamicMaterialInst);
		StartRotationIndicatorMeshComponent->SetMaterial(1, TranslucentDynamicMaterialInst);
		FullRotationHandleMeshComponent->SetMaterial(1, TranslucentDynamicMaterialInst);

	}

	DragOperationComponent->SetDragOperationClass(URotateOnAngleDragOperation::StaticClass());
}

void UPivotRotationGizmoHandleGroup::UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle,
	const TArray< UActorComponent* >& HoveringOverHandles, float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	// Call parent implementation
	Super::UpdateGizmoHandleGroup(LocalToWorld, LocalBounds, ViewLocation, bAllHandlesVisible, DraggingHandle, HoveringOverHandles, AnimationAlpha,
		GizmoScale, GizmoHoverScale, GizmoHoverAnimationDuration, bOutIsHoveringOrDraggingThisHandleGroup);

	bool bShowFullRotationDragHandle = false;
	UViewportWorldInteraction* WorldInteraction = OwningTransformGizmoActor->GetOwnerWorldInteraction();
	const ECoordSystem CoordSystem = WorldInteraction->GetTransformGizmoCoordinateSpace();
	const bool bIsTypeSupported = (OwningTransformGizmoActor->GetGizmoType() == EGizmoHandleTypes::All && GetShowOnUniversalGizmo()) || GetHandleType() == OwningTransformGizmoActor->GetGizmoType();
	const bool bSupportsCurrentCoordinateSpace = SupportsWorldCoordinateSpace() || CoordSystem != COORD_World;
	const bool bShowAnyRotationHandle = (bIsTypeSupported && bSupportsCurrentCoordinateSpace && bAllHandlesVisible);

	for (int32 HandleIndex = 0; HandleIndex < Handles.Num(); ++HandleIndex)
	{
		FGizmoHandle& Handle = Handles[HandleIndex];
		UStaticMeshComponent* GizmoHandleMeshComponent = Handle.HandleMesh;
		if (GizmoHandleMeshComponent != nullptr)	// Can be null if no handle for this specific placement
		{
			const bool bDraggingCurrentHandle = (DraggingHandle != nullptr && DraggingHandle == GizmoHandleMeshComponent);
			const bool bShouldShowHandle = bShowAnyRotationHandle && !bDraggingCurrentHandle;
			GizmoHandleMeshComponent->SetVisibility(bShouldShowHandle);
			GizmoHandleMeshComponent->SetCollisionEnabled(bShouldShowHandle ? ECollisionEnabled::QueryOnly : ECollisionEnabled::NoCollision);

			int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
			const FTransformGizmoHandlePlacement HandlePlacement = MakeHandlePlacementForIndex(HandleIndex);
			HandlePlacement.GetCenterHandleCountAndFacingAxisIndex(/* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex);

			if (bDraggingCurrentHandle)
			{
				bool bInitialDrag = false;
				bShowFullRotationDragHandle = true;
				FVector GizmoSpaceFacingAxisVector = GetAxisVector(FacingAxisIndex, HandlePlacement.Axes[FacingAxisIndex]);
				
				// Set the initial rotation
				if (!StartDragRotation.IsSet())
				{
					StartDragRotation = LocalToWorld.GetRotation();
					bInitialDrag = true;
				}
				
				// Set the root of the full rotation handles to the rotation we had when starting the drag.
				RootFullRotationHandleComponent->SetWorldRotation(StartDragRotation.GetValue());	

				FullRotationHandleMeshComponent->SetRelativeTransform(FTransform(GizmoSpaceFacingAxisVector.ToOrientationQuat(), FVector::ZeroVector, FVector(GizmoScale)));

				URotateOnAngleDragOperation* DragOperation = StaticCast<URotateOnAngleDragOperation*>(GetDragOperationComponent()->GetDragOperation());
				const FVector LocalIntersectPoint = DragOperation->GetLocalIntersectPointOnRotationGizmo();

				UpdateIndicator(RootDeltaRotationIndicatorComponent, LocalIntersectPoint, FacingAxisIndex);

				// Update the start indicator only if this was the first time dragging
				if (bInitialDrag)
				{
					UpdateIndicator(RootStartRotationIdicatorComponent, LocalIntersectPoint, FacingAxisIndex);

					FLinearColor Color = WorldInteraction->GetColor(UViewportWorldInteraction::EColors::GizmoHover);
					SetIndicatorColor(FullRotationHandleMeshComponent, Color);
					SetIndicatorColor(DeltaRotationIndicatorMeshComponent, Color);
					SetIndicatorColor(StartRotationIndicatorMeshComponent, Color);

					bInitialDrag = false;
				}
			}
			else if(DraggingHandle == nullptr)
			{
				int32 UpAxisIndex = 0;
				int32 RightAxisIndex = 0;
				FRotator Rotation;

				if (FacingAxisIndex == 0)
				{
					// X, up = Z, right = Y
					UpAxisIndex = 2;
					RightAxisIndex = 1;
					Rotation = FRotator::ZeroRotator;
				}
				else if (FacingAxisIndex == 1)
				{
					// Y, up = Z, right = X
					UpAxisIndex = 2;
					RightAxisIndex = 0;
					Rotation = FRotator(0, -90, 0);

				}
				else if (FacingAxisIndex == 2)
				{
					// Z, up = Y, right = X
					UpAxisIndex = 0;
					RightAxisIndex = 1;
					Rotation = FRotator(-90, 0, 0);
				}

				// Check on which side we are relative to the gizmo
				const FVector GizmoSpaceViewLocation = GetOwner()->GetTransform().InverseTransformPosition(ViewLocation);
				if (GizmoSpaceViewLocation[UpAxisIndex] < 0 && GizmoSpaceViewLocation[RightAxisIndex] < 0)
				{
					Rotation.Roll = 180;
				}
				else if (GizmoSpaceViewLocation[UpAxisIndex] < 0)
				{
					Rotation.Roll = 90;
				}
				else if (GizmoSpaceViewLocation[RightAxisIndex] < 0)
				{
					Rotation.Roll = -90;
				}

				const float GizmoHandleScale = GizmoScale * AnimationAlpha;

				// Set the final transform
				GizmoHandleMeshComponent->SetRelativeTransform(FTransform(Rotation, FVector::ZeroVector, FVector(GizmoHandleScale)));

				// Update material
				UpdateHandleColor(FacingAxisIndex, Handle, DraggingHandle, HoveringOverHandles);

				// Reset the start rotation
				StartDragRotation.Reset();
			}
		}
	}

	// Show or hide the visuals for when rotating
	ShowRotationVisuals(bShowFullRotationDragHandle);
}

EGizmoHandleTypes UPivotRotationGizmoHandleGroup::GetHandleType() const
{
	return EGizmoHandleTypes::Rotate;
}

void UPivotRotationGizmoHandleGroup::UpdateIndicator(USceneComponent* IndicatorRoot, const FVector& Direction, const uint32 FacingAxisIndex)
{
	double Y = 0, X = 0;
	if (FacingAxisIndex == 0)
	{
		Y = Direction.Y;
		X = Direction.Z;
	}
	else if (FacingAxisIndex == 1)
	{
		Y = -Direction.X;
		X = Direction.Z;
	}
	else if (FacingAxisIndex == 2)
	{
		Y = Direction.Y;
		X = -Direction.X;
	}

	const double Angle = FMath::RadiansToDegrees(FMath::Atan2(Y, X));
	IndicatorRoot->SetRelativeRotation(FRotator(0, 0, Angle));
}

void UPivotRotationGizmoHandleGroup::ShowRotationVisuals(const bool bInShow)
{
	FullRotationHandleMeshComponent->SetCollisionEnabled(bInShow == true ? ECollisionEnabled::QueryOnly : ECollisionEnabled::NoCollision);
	FullRotationHandleMeshComponent->SetVisibility(bInShow);
	StartRotationIndicatorMeshComponent->SetVisibility(bInShow);
	DeltaRotationIndicatorMeshComponent->SetVisibility(bInShow);
}

void UPivotRotationGizmoHandleGroup::SetupIndicator(USceneComponent* RootComponent, UGizmoHandleMeshComponent* IndicatorMeshComponent, UStaticMesh* Mesh)
{
	RootComponent->SetMobility(EComponentMobility::Movable);
	RootComponent->SetupAttachment(FullRotationHandleMeshComponent);

	IndicatorMeshComponent->SetCanEverAffectNavigation(false);
	IndicatorMeshComponent->SetStaticMesh(Mesh);
	IndicatorMeshComponent->SetMobility(EComponentMobility::Movable);
	IndicatorMeshComponent->SetupAttachment(RootComponent);
	IndicatorMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	IndicatorMeshComponent->SetVisibility(false);
	IndicatorMeshComponent->SetGenerateOverlapEvents(false);
	IndicatorMeshComponent->bCastDynamicShadow = true;
	IndicatorMeshComponent->bCastStaticShadow = false;
	IndicatorMeshComponent->bAffectDistanceFieldLighting = false;
	IndicatorMeshComponent->bAffectDynamicIndirectLighting = false;
	IndicatorMeshComponent->SetRelativeLocation(FVector(0, 0, 100));
}

void UPivotRotationGizmoHandleGroup::SetIndicatorColor(UStaticMeshComponent* InMeshComponent, const FLinearColor& InHandleColor)
{
	UMaterialInstanceDynamic* MID0 = Cast<UMaterialInstanceDynamic>(InMeshComponent->GetMaterial(0));
	UMaterialInstanceDynamic* MID1 = Cast<UMaterialInstanceDynamic>(InMeshComponent->GetMaterial(1));

	static FName StaticHandleColorParameter("Color");
	MID0->SetVectorParameterValue(StaticHandleColorParameter, InHandleColor);
	MID1->SetVectorParameterValue(StaticHandleColorParameter, InHandleColor);
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
