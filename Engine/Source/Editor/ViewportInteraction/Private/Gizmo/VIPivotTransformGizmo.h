// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "VIBaseTransformGizmo.h"
#include "VIGizmoHandle.h"
#include "VIPivotTransformGizmo.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


/**
 * A transform gizmo on the pivot that allows you to interact with selected objects by moving, scaling and rotating.
 */
UCLASS()
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") APivotTransformGizmo : public ABaseTransformGizmo
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	APivotTransformGizmo();
	
	/** Called by the world interaction system after we've been spawned into the world, to allow
	    us to create components and set everything up nicely for the selected objects that we'll be
		used to manipulate */
	virtual void UpdateGizmo(const EGizmoHandleTypes InGizmoType, const ECoordSystem InGizmoCoordinateSpace, const FTransform& InLocalToWorld, const FBox& InLocalBounds,
		const FVector& InViewLocation, const float InScaleMultiplier, bool bInAllHandlesVisible, const bool bInAllowRotationAndScaleHandles, class UActorComponent* DraggingHandle,
		const TArray<UActorComponent*>& InHoveringOverHandles, const float InGizmoHoverScale, const float InGizmoHoverAnimationDuration) override;

private:
	/** Uniform scale handle group component */
	UPROPERTY()
	TObjectPtr<class UUniformScaleGizmoHandleGroup> UniformScaleGizmoHandleGroup;

	/** Translation handle group component */
	UPROPERTY()
	TObjectPtr<class UPivotTranslationGizmoHandleGroup> TranslationGizmoHandleGroup;
	
	/** Scale handle group component */
	UPROPERTY()
	TObjectPtr<class UPivotScaleGizmoHandleGroup> ScaleGizmoHandleGroup;
	
	/** Plane translation handle group component */
	UPROPERTY()
	TObjectPtr<class UPivotPlaneTranslationGizmoHandleGroup> PlaneTranslationGizmoHandleGroup;

	/** Rotation handle group component */
	UPROPERTY()
	TObjectPtr<class UPivotRotationGizmoHandleGroup> RotationGizmoHandleGroup;

	/** Stretch handle group component */
	UPROPERTY()
	TObjectPtr<class UStretchGizmoHandleGroup> StretchGizmoHandleGroup;

	/** The alpha for gizmo animation when aiming at it with a laser */
	float AimingAtGizmoScaleAlpha;

	/** Handle from previous tick that was dragged */
	UPROPERTY()
	TObjectPtr<UActorComponent> LastDraggingHandle;
};

/**
 * Axis Gizmo handle for translating
 */
UCLASS()
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UPivotTranslationGizmoHandleGroup : public UAxisGizmoHandleGroup
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UPivotTranslationGizmoHandleGroup();

	/** Updates the gizmo handles */
	virtual void UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
		float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup ) override;

	/** Gets the GizmoType for this Gizmo handle */
	virtual EGizmoHandleTypes GetHandleType() const override;
};

/**
 * Axis Gizmo handle for scaling
 */
UCLASS()
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UPivotScaleGizmoHandleGroup : public UAxisGizmoHandleGroup
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UPivotScaleGizmoHandleGroup();

	/** Updates the gizmo handles */
	virtual void UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds,  const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
		float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup ) override;

	/** Gets the GizmoType for this Gizmo handle */
	virtual EGizmoHandleTypes GetHandleType() const override;

	/** Returns true if this type of handle is allowed with world space gizmos */
	virtual bool SupportsWorldCoordinateSpace() const override;
};

/**
 * Axis Gizmo handle for plane translation
 */
UCLASS()
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UPivotPlaneTranslationGizmoHandleGroup : public UAxisGizmoHandleGroup
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UPivotPlaneTranslationGizmoHandleGroup();

	/** Updates the gizmo handles */
	virtual void UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
		float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup ) override;

	/** Gets the GizmoType for this Gizmo handle */
	virtual EGizmoHandleTypes GetHandleType() const override;
};


/**
 * Axis Gizmo handle for rotation
 */
UCLASS()
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UPivotRotationGizmoHandleGroup : public UAxisGizmoHandleGroup
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UPivotRotationGizmoHandleGroup();

	/** Updates the gizmo handles */
	virtual void UpdateGizmoHandleGroup(const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
		float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup ) override;

	/** Gets the GizmoType for this Gizmo handle */
	virtual EGizmoHandleTypes GetHandleType() const override;

private:

	/** Updates the root of an indicator to rotate the indicator itself */
	void UpdateIndicator(USceneComponent* IndicatorRoot, const FVector& Direction, const uint32 FacingAxisIndex);

	/** Make the components visible when dragging rotation */
	void ShowRotationVisuals(const bool bInShow);

	void SetupIndicator(USceneComponent* RootComponent, UGizmoHandleMeshComponent* IndicatorMeshComponent, UStaticMesh* Mesh);
	
	void SetIndicatorColor(UStaticMeshComponent* InMeshComponent, const FLinearColor& InHandleColor);

	/** Root component of all the mesh components that are used to visualize the rotation when dragging */
	UPROPERTY()
	TObjectPtr<USceneComponent> RootFullRotationHandleComponent;

	/** When dragging a rotation handle the full rotation circle appears */
	UPROPERTY()
	TObjectPtr<class UGizmoHandleMeshComponent> FullRotationHandleMeshComponent;

	/** The mesh that indicated the start rotation */
	UPROPERTY()
	TObjectPtr<class UGizmoHandleMeshComponent> StartRotationIndicatorMeshComponent;

	/** The root component of the start rotation indicator */
	UPROPERTY()
	TObjectPtr<USceneComponent> RootStartRotationIdicatorComponent;

	/** The mesh that indicated the delta rotation */
	UPROPERTY()
	TObjectPtr<class UGizmoHandleMeshComponent> DeltaRotationIndicatorMeshComponent;

	/** The root component of the delta rotation indicator */
	UPROPERTY()
	TObjectPtr<USceneComponent> RootDeltaRotationIndicatorComponent;

	/** The rotation when starting to drag the gizmo */
	TOptional<FQuat> StartDragRotation;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
