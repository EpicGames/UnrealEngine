// Copyright Epic Games, Inc. All Rights Reserved.

#include "VIStretchGizmoHandle.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/StaticMesh.h"
#include "VIBaseTransformGizmo.h"
#include "ViewportInteractionDragOperations.h"
#include "VIGizmoHandleMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VIStretchGizmoHandle)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UStretchGizmoHandleGroup::UStretchGizmoHandleGroup()
	: Super()
{
	UStaticMesh* StretchingHandleMesh = nullptr;
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> ObjectFinder( TEXT( "/Engine/VREditor/TransformGizmo/PlaneTranslationHandle" ) );
		StretchingHandleMesh = ObjectFinder.Object;
		check( StretchingHandleMesh != nullptr );
	}

	UStaticMesh* BoundingBoxCornerMesh = nullptr;
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> ObjectFinder( TEXT( "/Engine/VREditor/TransformGizmo/BoundingBoxCorner" ) );
		BoundingBoxCornerMesh = ObjectFinder.Object;
		check( BoundingBoxCornerMesh != nullptr );
	}

	UStaticMesh* BoundingBoxEdgeMesh = nullptr;
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> ObjectFinder( TEXT( "/Engine/VREditor/TransformGizmo/BoundingBoxEdge" ) );
		BoundingBoxEdgeMesh = ObjectFinder.Object;
		check( BoundingBoxEdgeMesh != nullptr );
	}

	const bool bAllowGizmoLighting = false;	// @todo vreditor: Not sure if we want this for gizmos or not yet.  Needs feedback.  Also they're translucent right now.

	for (int32 X = 0; X < 3; ++X)
	{
		for (int32 Y = 0; Y < 3; ++Y)
		{
			for (int32 Z = 0; Z < 3; ++Z)
			{
				FTransformGizmoHandlePlacement HandlePlacement = GetHandlePlacement( X, Y, Z );

				int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
				HandlePlacement.GetCenterHandleCountAndFacingAxisIndex( /* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex );

				// Don't allow translation/stretching/rotation from the origin
				if (CenterHandleCount < 3)
				{
					const FString HandleName = MakeHandleName( HandlePlacement );

					// Stretching handle
					if (CenterHandleCount != 1)	// @todo vreditor: Remove this line if we want to re-enable support for edge stretching handles (rather than only corners).  We disabled this because they sort of got in the way of the rotation gizmo, and weren't very popular to use.
					{
						FString ComponentName = HandleName + TEXT( "StretchingHandle" );

						UStaticMesh* Mesh = nullptr;

						if (CenterHandleCount == 0)	// Corner?
						{
							Mesh = BoundingBoxCornerMesh;
						}
						else if (CenterHandleCount == 1)	// Edge?
						{
							Mesh = BoundingBoxEdgeMesh;
						}
						else  // Face
						{
							Mesh = StretchingHandleMesh;
						}

						CreateAndAddMeshHandle( Mesh, ComponentName, HandlePlacement );
					}
				}
			}
		}
	}

	DragOperationComponent->SetDragOperationClass(UStretchGizmoHandleDragOperation::StaticClass());
}

void UStretchGizmoHandleGroup::UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
	float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	// Call parent implementation (updates hover animation)
	Super::UpdateGizmoHandleGroup(LocalToWorld, LocalBounds, ViewLocation, bAllHandlesVisible, DraggingHandle, HoveringOverHandles,
		AnimationAlpha, GizmoScale, GizmoHoverScale, GizmoHoverAnimationDuration, bOutIsHoveringOrDraggingThisHandleGroup );

	for (int32 HandleIndex = 0; HandleIndex < Handles.Num(); ++HandleIndex)
	{
		FGizmoHandle& Handle = Handles[ HandleIndex ];

		UStaticMeshComponent* StretchingHandle = Handle.HandleMesh;
		if (StretchingHandle != nullptr)	// Can be null if no handle for this specific placement
		{
			const FTransformGizmoHandlePlacement HandlePlacement = MakeHandlePlacementForIndex( HandleIndex );

			float GizmoHandleScale = GizmoScale;

			const float OffsetFromSide = GizmoHandleScale *
				(0.0f +	// @todo vreditor tweak: Hard coded handle offset from side of primitive
				(1.0f - AnimationAlpha) * 10.0f);	// @todo vreditor tweak: Animation offset

			// Make the handle bigger while hovered (but don't affect the offset -- we want it to scale about it's origin)
			GizmoHandleScale *= FMath::Lerp( 1.0f, GizmoHoverScale, Handle.HoverAlpha );

			FVector HandleRelativeLocation = FVector::ZeroVector;
			for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
			{
				if (HandlePlacement.Axes[AxisIndex] == ETransformGizmoHandleDirection::Negative)	// Negative direction
				{
					HandleRelativeLocation[AxisIndex] = LocalBounds.Min[AxisIndex] - OffsetFromSide;
				}
				else if (HandlePlacement.Axes[AxisIndex] == ETransformGizmoHandleDirection::Positive)	// Positive direction
				{
					HandleRelativeLocation[AxisIndex] = LocalBounds.Max[AxisIndex] + OffsetFromSide;
				}
				else // ETransformGizmoHandleDirection::Center
				{
					HandleRelativeLocation[AxisIndex] = LocalBounds.GetCenter()[AxisIndex];
				}
			}

			StretchingHandle->SetRelativeLocation( HandleRelativeLocation );

			int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
			HandlePlacement.GetCenterHandleCountAndFacingAxisIndex( /* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex );

			FRotator Rotator = FRotator::ZeroRotator;
			{
				// Back bottom left
				if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Negative)
				{
					Rotator.Yaw = 0.0f;
					Rotator.Pitch = 0.0f;
				}

				// Back bottom right
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Negative)
				{
					Rotator.Yaw = -90.0f;
					Rotator.Pitch = 0.0f;
				}

				// Back top left
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Positive)
				{
					Rotator.Yaw = 0.0f;
					Rotator.Pitch = -90.0f;
				}

				// Back top right
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Positive)
				{
					Rotator.Yaw = -90.0f;
					Rotator.Pitch = -90.0f;
				}

				// Front bottom left
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Negative)
				{
					Rotator.Yaw = 0.0f;
					Rotator.Pitch = 90.0f;
				}

				// Front bottom right
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Negative)
				{
					Rotator.Yaw = 90.0f;
					Rotator.Pitch = 90.0f;
				}

				// Front top left
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Positive)
				{
					Rotator.Yaw = 0.0f;
					Rotator.Pitch = -180.0f;
				}

				// Front top right
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[1] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[2] == ETransformGizmoHandleDirection::Positive)
				{
					Rotator.Yaw = 180.0f;
					Rotator.Pitch = -90.0f;
				}

				// Back left/right edge
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[1] != ETransformGizmoHandleDirection::Center)
				{
					Rotator.Yaw = 0.0f;
					Rotator.Pitch = 90.0f;
				}

				// Back bottom/top edge
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Negative && HandlePlacement.Axes[2] != ETransformGizmoHandleDirection::Center)
				{
					Rotator.Yaw = 90.0f;
					Rotator.Pitch = 0.0f;
				}

				// Front left/right edge
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[1] != ETransformGizmoHandleDirection::Center)
				{
					Rotator.Yaw = 0.0f;
					Rotator.Pitch = 90.0f;
				}

				// Front bottom/top edge
				else if (HandlePlacement.Axes[0] == ETransformGizmoHandleDirection::Positive && HandlePlacement.Axes[2] != ETransformGizmoHandleDirection::Center)
				{
					Rotator.Yaw = 90.0f;
					Rotator.Pitch = 0.0f;
				}

				else
				{
					// Facing out from center of a face
					if (CenterHandleCount == 2)
					{
						const FQuat GizmoSpaceOrientation = GetAxisVector( FacingAxisIndex, HandlePlacement.Axes[FacingAxisIndex] ).ToOrientationQuat();
						Rotator = GizmoSpaceOrientation.Rotator();
					}

					else
					{
						// One of the left/right bottom or top edges
					}
				}
			}

			StretchingHandle->SetRelativeRotation( Rotator );

			StretchingHandle->SetRelativeScale3D( FVector( GizmoHandleScale ) );

			// Update material
			UpdateHandleColor( FacingAxisIndex, Handle, DraggingHandle, HoveringOverHandles );
		}
	}
}

EGizmoHandleTypes UStretchGizmoHandleGroup::GetHandleType() const
{
	return EGizmoHandleTypes::Scale;
}

bool UStretchGizmoHandleGroup::SupportsWorldCoordinateSpace() const
{
	// Stretching only works with local space gizmos
	return false;
}

void UStretchGizmoHandleDragOperation::ExecuteDrag(FDraggingTransformableData& DraggingData)
{
	// We only support stretching by a handle currently
	const FTransformGizmoHandlePlacement& HandlePlacement = DraggingData.OptionalHandlePlacement.GetValue();

	const FVector PassGizmoSpaceDraggedTo = DraggingData.GizmoStartTransform.InverseTransformPositionNoScale(DraggingData.PassDraggedTo);

	FBox NewGizmoLocalBounds = DraggingData.GizmoStartLocalBounds;
	FVector GizmoSpacePivotLocation = FVector::ZeroVector;
	for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		// Figure out how much the gizmo bounds changes
		if (HandlePlacement.Axes[AxisIndex] == ETransformGizmoHandleDirection::Negative)	// Negative direction
		{
			GizmoSpacePivotLocation[AxisIndex] = DraggingData.GizmoStartLocalBounds.Max[AxisIndex];
			NewGizmoLocalBounds.Min[AxisIndex] = DraggingData.GizmoStartLocalBounds.Min[AxisIndex] + PassGizmoSpaceDraggedTo[AxisIndex];

		}
		else if (HandlePlacement.Axes[AxisIndex] == ETransformGizmoHandleDirection::Positive)	// Positive direction
		{
			GizmoSpacePivotLocation[AxisIndex] = DraggingData.GizmoStartLocalBounds.Min[AxisIndex];
			NewGizmoLocalBounds.Max[AxisIndex] = DraggingData.GizmoStartLocalBounds.Max[AxisIndex] + PassGizmoSpaceDraggedTo[AxisIndex];
		}
		else
		{
			// Must be ETransformGizmoHandleDirection::Center.  
			GizmoSpacePivotLocation[AxisIndex] = DraggingData.GizmoStartLocalBounds.GetCenter()[AxisIndex];
		}
	}

	const FVector GizmoStartLocalSize = DraggingData.GizmoStartLocalBounds.GetSize();
	const FVector NewGizmoLocalSize = NewGizmoLocalBounds.GetSize();

	FVector NewGizmoLocalScaleFromStart = FVector(1.0f);
	for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (!FMath::IsNearlyZero(GizmoStartLocalSize[AxisIndex]))
		{
			NewGizmoLocalScaleFromStart[AxisIndex] = NewGizmoLocalSize[AxisIndex] / GizmoStartLocalSize[AxisIndex];
		}
		else
		{
			// Zero scale.  This is allowed in Unreal, for better or worse.
			NewGizmoLocalScaleFromStart[AxisIndex] = 0.0f;
		}
	}

	// Stretch and reposition the gizmo!
	{
		FTransform& PassGizmoTargetTransform  = DraggingData.OutGizmoUnsnappedTargetTransform;

		{
			const FVector GizmoSpaceTransformableStartLocation = DraggingData.GizmoStartTransform.InverseTransformPositionNoScale(DraggingData.GizmoStartTransform.GetLocation());
			const FVector NewGizmoSpaceLocation = (GizmoSpaceTransformableStartLocation - GizmoSpacePivotLocation) * NewGizmoLocalScaleFromStart + GizmoSpacePivotLocation;
			PassGizmoTargetTransform.SetLocation(DraggingData.GizmoStartTransform.TransformPosition(NewGizmoSpaceLocation));
		}

		// @todo vreditor: This scale is still in gizmo space, but we're setting it in world space
		PassGizmoTargetTransform.SetScale3D(DraggingData.GizmoStartTransform.GetScale3D() * NewGizmoLocalScaleFromStart);

		DraggingData.bOutMovedTransformGizmo = true;
		DraggingData.bOutShouldApplyVelocitiesFromDrag = false;
		DraggingData.bOutScaled = true;
		DraggingData.bAllowSnap = false;
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
