// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "VIGizmoHandle.h"
#include "ViewportDragOperation.h"
#include "VIStretchGizmoHandle.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


enum class EGizmoHandleTypes : uint8;

/**
 * Gizmo handle for stretching/scaling
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UStretchGizmoHandleGroup : public UGizmoHandleGroup
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UE_API UStretchGizmoHandleGroup();
	
	/** Updates the Gizmo handles */
	UE_API virtual void UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
		float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup ) override;

	/** Gets the GizmoType for this Gizmo handle */
	UE_API virtual EGizmoHandleTypes GetHandleType() const override;

	/** Returns true if this type of handle is allowed with world space gizmos */
	UE_API virtual bool SupportsWorldCoordinateSpace() const override;
};

UCLASS()
class UStretchGizmoHandleDragOperation : public UViewportDragOperation
{
	GENERATED_BODY()

public:

	// IViewportDragOperation
	virtual void ExecuteDrag(struct FDraggingTransformableData& DraggingData) override;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
