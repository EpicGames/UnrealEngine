// Copyright Epic Games, Inc. All Rights Reserved.

#include "VIUniformScaleGizmoHandle.h"
#include "UObject/ConstructorHelpers.h"
#include "VIGizmoHandleMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "VIBaseTransformGizmo.h"
#include "ViewportWorldInteraction.h"
#include "ViewportInteractionDragOperations.h"
#include "ViewportInteractionAssetContainer.h"
#include "UObject/StrongObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VIUniformScaleGizmoHandle)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UUniformScaleGizmoHandleGroup::UUniformScaleGizmoHandleGroup()
	: Super(),
	bUsePivotAsLocation( true )
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	// Setup uniform scaling
	TStrongObjectPtr<const UViewportInteractionAssetContainer> AssetContainer(UViewportWorldInteraction::LoadAssetContainer()); 
	UStaticMesh* UniformScaleMesh = AssetContainer->UniformScaleHandleMesh;
	check( UniformScaleMesh != nullptr );

	UGizmoHandleMeshComponent* UniformScaleHandle = CreateMeshHandle( UniformScaleMesh, FString( "UniformScaleHandle" ) );
	check( UniformScaleHandle != nullptr );

	FGizmoHandle& NewHandle = *new( Handles ) FGizmoHandle();
	NewHandle.HandleMesh = UniformScaleHandle;

	DragOperationComponent->SetDragOperationClass(UUniformScaleDragOperation::StaticClass());
}

void UUniformScaleGizmoHandleGroup::UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
	float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup )
{
	// Call parent implementation (updates hover animation)
	Super::UpdateGizmoHandleGroup( LocalToWorld, LocalBounds, ViewLocation, bAllHandlesVisible, DraggingHandle, HoveringOverHandles, 
		AnimationAlpha, GizmoScale, GizmoHoverScale, GizmoHoverAnimationDuration, bOutIsHoveringOrDraggingThisHandleGroup );

	FGizmoHandle& Handle = Handles[ 0 ];
	UStaticMeshComponent* UniformScaleHandle = Handle.HandleMesh;
	if (UniformScaleHandle != nullptr)	// Can be null if no handle for this specific placement
	{
		FVector HandleRelativeLocation = FVector::ZeroVector;
		if ( !bUsePivotAsLocation )
		{
			UniformScaleHandle->SetRelativeLocation( HandleRelativeLocation );
		}

		float GizmoHandleScale = GizmoScale;

		// Make the handle bigger while hovered (but don't affect the offset -- we want it to scale about it's origin)
		GizmoHandleScale *= FMath::Lerp( 1.0f, GizmoHoverScale, Handle.HoverAlpha );

		UniformScaleHandle->SetRelativeScale3D( FVector( GizmoHandleScale ) );

		// Update material
		{
			if (!UniformScaleHandle->GetMaterial( 0 )->IsA( UMaterialInstanceDynamic::StaticClass() ))
			{
				UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create( GizmoMaterial, this );
				UniformScaleHandle->SetMaterial( 0, MID );
			}
			if (!UniformScaleHandle->GetMaterial( 1 )->IsA( UMaterialInstanceDynamic::StaticClass() ))
			{
				UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create( TranslucentGizmoMaterial, this );
				UniformScaleHandle->SetMaterial( 1, MID );
			}
			UMaterialInstanceDynamic* MID0 = CastChecked<UMaterialInstanceDynamic>( UniformScaleHandle->GetMaterial( 0 ) );
			UMaterialInstanceDynamic* MID1 = CastChecked<UMaterialInstanceDynamic>( UniformScaleHandle->GetMaterial( 1 ) );

			ABaseTransformGizmo* GizmoActor = CastChecked<ABaseTransformGizmo>( GetOwner() );
			if( GizmoActor )
			{
				UViewportWorldInteraction* WorldInteraction = GizmoActor->GetOwnerWorldInteraction();
				if( WorldInteraction && WorldInteraction->IsActive() )
				{
					FLinearColor HandleColor = WorldInteraction->GetColor( UViewportWorldInteraction::EColors::DefaultColor );
					if (UniformScaleHandle == DraggingHandle)
					{
						HandleColor = WorldInteraction->GetColor( UViewportWorldInteraction::EColors::GizmoDragging );
					}
					else if (HoveringOverHandles.Contains( UniformScaleHandle ))
					{
						HandleColor = FLinearColor::LerpUsingHSV( HandleColor, WorldInteraction->GetColor( UViewportWorldInteraction::EColors::GizmoHover ), Handle.HoverAlpha );
					}

					MID0->SetVectorParameterValue( "Color", HandleColor );
					MID1->SetVectorParameterValue( "Color", HandleColor );
				}
			}
		}
	}
}

EGizmoHandleTypes UUniformScaleGizmoHandleGroup::GetHandleType() const
{
	return EGizmoHandleTypes::Scale;
}

void UUniformScaleGizmoHandleGroup::SetUsePivotPointAsLocation( const bool bInUsePivotAsLocation )
{
	bUsePivotAsLocation = bInUsePivotAsLocation;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
