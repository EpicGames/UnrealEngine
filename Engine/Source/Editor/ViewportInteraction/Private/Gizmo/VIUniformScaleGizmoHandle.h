// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "VIGizmoHandle.h"
#include "VIUniformScaleGizmoHandle.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


enum class EGizmoHandleTypes : uint8;

/**
 * Gizmo handle for uniform scaling
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UUniformScaleGizmoHandleGroup : public UGizmoHandleGroup
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UE_API UUniformScaleGizmoHandleGroup();
	
	/** Updates the Gizmo handles */
	UE_API virtual void UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
		float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup ) override;

	/** Gets the GizmoType for this Gizmo handle */
	UE_API virtual EGizmoHandleTypes GetHandleType() const override;

	/** Sets if the pivot point is used as location for the handle */
	UE_API void SetUsePivotPointAsLocation( const bool bInUsePivotAsLocation );

private:

	/** If the pivot point is used for the uniform scaling handle */
	bool bUsePivotAsLocation;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
