// Copyright Epic Games, Inc. All Rights Reserved.

#include "MouseCursorInteractor.h"
#include "SceneView.h"
#include "EditorViewportClient.h"
#include "ViewportWorldInteraction.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MouseCursorInteractor)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UMouseCursorInteractor::UMouseCursorInteractor()
	: UViewportInteractor()
{
	// Grabber spheres don't really work well with mouse cursor interactors, because the origin of the
	// interactor is right on the near view plane.
	bAllowGrabberSphere = false;
}


void UMouseCursorInteractor::Init()
{
	KeyToActionMap.Reset();

	// Setup keys
	{
		AddKeyAction( EKeys::LeftMouseButton, FViewportActionKeyInput( ViewportWorldActionTypes::SelectAndMove ) );
	}
}


void UMouseCursorInteractor::PollInput()
{
	InteractorData.LastTransform = InteractorData.Transform;
	InteractorData.LastRoomSpaceTransform = InteractorData.RoomSpaceTransform;

	// Make sure we have a valid viewport with a cursor over it, and that the viewport's world is the same as ours
	FEditorViewportClient* ViewportClientPtr = WorldInteraction->GetDefaultOptionalViewportClient();
	if( ViewportClientPtr != nullptr && ViewportClientPtr->GetWorld() == WorldInteraction->GetWorld() )
	{
		FEditorViewportClient& ViewportClient = *ViewportClientPtr;

		bIsControlKeyPressed = ViewportClient.Viewport->KeyState( EKeys::LeftControl ) || ViewportClient.Viewport->KeyState( EKeys::RightControl );

		// Only if we're not tracking (RMB looking)
		if( !ViewportClient.IsTracking() )
		{
			FViewport* Viewport = ViewportClient.Viewport;

			// Make sure we have a valid viewport, otherwise we won't be able to construct an FSceneView.  The first time we're ticked we might not be properly setup. (@todo viewportinteraction)
			if( Viewport != nullptr && Viewport->GetSizeXY().GetMin() > 0 )
			{
				const int32 ViewportInteractX = Viewport->GetMouseX();
				const int32 ViewportInteractY = Viewport->GetMouseY();

				FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues(
					Viewport,
					ViewportClient.GetScene(),
					ViewportClient.EngineShowFlags )
					.SetRealtimeUpdate( ViewportClient.IsRealtime() ) );
				FSceneView* SceneView = ViewportClient.CalcSceneView( &ViewFamily );

				const FViewportCursorLocation MouseViewportRay( SceneView, &ViewportClient, ViewportInteractX, ViewportInteractY );

				FVector RayOrigin = MouseViewportRay.GetOrigin();

				// If we're dealing with an orthographic view, push the origin of the ray backward along the viewport forward axis
				// to make sure that we can select objects that are behind the origin!
				if( !ViewportClient.IsPerspective() )
				{
					const float HalfLaserPointerLength = this->GetLaserPointerMaxLength() * 0.5f;
					RayOrigin -= MouseViewportRay.GetDirection() * HalfLaserPointerLength;
				}

				InteractorData.Transform = FTransform( MouseViewportRay.GetDirection().ToOrientationQuat(), RayOrigin, FVector( 1.0f ) );
			}
		}
	}
	InteractorData.RoomSpaceTransform = InteractorData.Transform * WorldInteraction->GetRoomTransform().Inverse();
}


bool UMouseCursorInteractor::IsModifierPressed() const
{
	return bIsControlKeyPressed;
}

bool UMouseCursorInteractor::AllowLaserSmoothing() const
{
	return false;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
