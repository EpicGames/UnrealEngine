// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportDragOperation.h"

#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportDragOperation)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UViewportDragOperationComponent::UViewportDragOperationComponent() : 
	Super(),
	DragOperation(nullptr)
{

}

UViewportDragOperation* UViewportDragOperationComponent::GetDragOperation()
{
	return DragOperation;
}

void UViewportDragOperationComponent::SetDragOperationClass( const TSubclassOf<UViewportDragOperation> InDragOperation )
{
	DragOperationSubclass = InDragOperation;
}

void UViewportDragOperationComponent::StartDragOperation()
{
	if ( DragOperationSubclass )
	{
		// Reset the drag operation to make sure we start a new one
		ClearDragOperation();
		// Create the drag object with the latest class
		DragOperation = NewObject<UViewportDragOperation>( ( UObject* ) GetTransientPackage(), DragOperationSubclass );
	}
}

void UViewportDragOperationComponent::ClearDragOperation()
{
	if (DragOperation)
	{
		DragOperation->MarkAsGarbage();
	}

	DragOperation = nullptr;
}

bool UViewportDragOperationComponent::IsDragging() const
{
	return DragOperation != nullptr;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
