// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractionDragOperations.h"
#include "ViewportInteractionTypes.h"
#include "VIGizmoHandle.h"
#include "ViewportInteractor.h"
#include "ViewportWorldInteraction.h"
#include "SnappingUtils.h"
#include "ViewportInteractor.h"
#include "UnrealWidgetFwd.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportInteractionDragOperations)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VI
{
	static FAutoConsoleVariable ScaleSensitivity(TEXT("VI.ScaleSensitivity"), 0.005f, TEXT("Sensitivity for scaling"));
}

void UTranslationDragOperation::ExecuteDrag(FDraggingTransformableData& DraggingData)
{
	// Translate the gizmo!
	DraggingData.OutGizmoUnsnappedTargetTransform.SetLocation(DraggingData.PassDraggedTo);
	DraggingData.bOutMovedTransformGizmo = true;
	DraggingData.bOutShouldApplyVelocitiesFromDrag = true;
	DraggingData.bOutTranslated = true;
}

UPlaneTranslationDragOperation::UPlaneTranslationDragOperation()
{
	bPlaneConstraint = true;
}

void UPlaneTranslationDragOperation::ExecuteDrag(struct FDraggingTransformableData& DraggingData)
{
	// Translate the gizmo!
	DraggingData.OutGizmoUnsnappedTargetTransform.SetLocation(DraggingData.PassDraggedTo);
	DraggingData.bOutMovedTransformGizmo = true;
	DraggingData.bOutShouldApplyVelocitiesFromDrag = true;
	DraggingData.bOutTranslated = true;
}

URotateOnAngleDragOperation::URotateOnAngleDragOperation():
	Super(),
	StartDragAngleOnRotation(),
	DraggingRotationHandleDirection()
{

}

void URotateOnAngleDragOperation::ExecuteDrag(FDraggingTransformableData& DraggingData)
{
	const FTransformGizmoHandlePlacement& HandlePlacement = DraggingData.OptionalHandlePlacement.GetValue();
	const FTransform& GizmoStartTransform = DraggingData.GizmoStartTransform;

	int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
	HandlePlacement.GetCenterHandleCountAndFacingAxisIndex( /* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex);

	FVector GizmoSpaceFacingAxisVector = UGizmoHandleGroup::GetAxisVector(FacingAxisIndex, HandlePlacement.Axes[FacingAxisIndex]);

	USceneComponent* DraggingTransformGizmoComponent = DraggingData.Interactor->GetInteractorData().DraggingTransformGizmoComponent.Get();
	if (DraggingTransformGizmoComponent)
	{
		const FTransform WorldToGizmo = GizmoStartTransform.Inverse();

		FTransform NewGizmoToWorld;
		{
			if (!DraggingRotationHandleDirection.IsSet())
			{
				DraggingRotationHandleDirection = DraggingTransformGizmoComponent->GetComponentTransform().GetRotation().Vector();
				DraggingRotationHandleDirection->Normalize();
			}

			//@todo ViewportInteraction: Use UViewportWorldInteraction::ComputeConstrainedDragDeltaFromStart instead of calculating where the laser hit on the plane with this intersect.
			// Get the laser pointer intersection on the plane of the handle
			const FPlane RotationPlane = FPlane(GizmoStartTransform.GetLocation(), DraggingRotationHandleDirection.GetValue());
			const ECoordSystem CoordSystem = DraggingData.WorldInteraction->GetTransformGizmoCoordinateSpace();
			const FVector LaserImpactOnRotationPlane = FMath::LinePlaneIntersection(DraggingData.LaserPointerStart, DraggingData.LaserPointerStart + DraggingData.LaserPointerDirection, RotationPlane);

			{
				FTransform GizmoTransformNoRotation = FTransform(FRotator::ZeroRotator, GizmoStartTransform.GetLocation());
				if (CoordSystem == COORD_Local)
				{
					GizmoTransformNoRotation.SetRotation(GizmoStartTransform.GetRotation());
				}

				LocalIntersectPointOnRotationGizmo = GizmoTransformNoRotation.InverseTransformPositionNoScale(LaserImpactOnRotationPlane);
			}

			// Set output for hover point
			DraggingData.OutUnsnappedDraggedTo = LaserImpactOnRotationPlane;

			// Relative offset of the intersection on the plane
			const FVector GizmoSpaceLaserImpactOnRotationPlane = WorldToGizmo.TransformPosition(LaserImpactOnRotationPlane);
			FVector RotatedIntersectLocationOnPlane;
			if (CoordSystem == COORD_Local)
			{
				RotatedIntersectLocationOnPlane = GizmoSpaceFacingAxisVector.Rotation().UnrotateVector(GizmoSpaceLaserImpactOnRotationPlane);
			}
			else
			{
				RotatedIntersectLocationOnPlane = GizmoStartTransform.TransformVector(GizmoSpaceFacingAxisVector).Rotation().UnrotateVector(GizmoSpaceLaserImpactOnRotationPlane);
			}

			// Get the angle between the center and the intersected point
			float AngleToIntersectedLocation = (float)FMath::Atan2(RotatedIntersectLocationOnPlane.Y, RotatedIntersectLocationOnPlane.Z);
			if (!StartDragAngleOnRotation.IsSet())
			{
				StartDragAngleOnRotation = AngleToIntersectedLocation;
			}

			// Delta rotation in gizmo space between the starting and the intersection rotation
			const float AngleDeltaRotationFromStart = FMath::FindDeltaAngleRadians(AngleToIntersectedLocation, StartDragAngleOnRotation.GetValue());
			const FQuat GizmoSpaceDeltaRotation = FQuat(GizmoSpaceFacingAxisVector, AngleDeltaRotationFromStart);

			const FTransform GizmoSpaceRotatedTransform(GizmoSpaceDeltaRotation);
			NewGizmoToWorld = GizmoSpaceRotatedTransform * GizmoStartTransform;
		}

		// Rotate the gizmo!
		DraggingData.OutGizmoUnsnappedTargetTransform = NewGizmoToWorld;
		DraggingData.bOutMovedTransformGizmo = true;
		DraggingData.bOutShouldApplyVelocitiesFromDrag = true;
		DraggingData.bOutRotated = true;
	}
}

FVector URotateOnAngleDragOperation::GetLocalIntersectPointOnRotationGizmo() const
{	
	return LocalIntersectPointOnRotationGizmo;
}

void UScaleDragOperation::ExecuteDrag(struct FDraggingTransformableData& DraggingData)
{
	const FTransformGizmoHandlePlacement& HandlePlacement = DraggingData.OptionalHandlePlacement.GetValue();
	int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
	HandlePlacement.GetCenterHandleCountAndFacingAxisIndex(/* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex);

	const FVector PassGizmoSpaceDraggedTo = DraggingData.GizmoStartTransform.InverseTransformPositionNoScale(DraggingData.PassDraggedTo);
	double AddedScaleOnAxis = PassGizmoSpaceDraggedTo[FacingAxisIndex] * VI::ScaleSensitivity->GetFloat();

	// Invert if we we are scaling on the negative side of the gizmo
	USceneComponent* DraggingTransformGizmoComponent = DraggingData.Interactor->GetInteractorData().DraggingTransformGizmoComponent.Get();
	if (DraggingTransformGizmoComponent && DraggingTransformGizmoComponent->GetRelativeTransform().GetLocation()[FacingAxisIndex] < 0)
	{
		AddedScaleOnAxis *= -1;
	}

	FVector NewScale = DraggingData.GizmoStartTransform.GetScale3D();
	NewScale[FacingAxisIndex] += AddedScaleOnAxis;
	DraggingData.OutGizmoUnsnappedTargetTransform.SetScale3D(NewScale);

	DraggingData.bOutMovedTransformGizmo = true;
	DraggingData.bOutShouldApplyVelocitiesFromDrag = true;
	DraggingData.bOutScaled = true;
}

void UUniformScaleDragOperation::ExecuteDrag(struct FDraggingTransformableData& DraggingData)
{
	//Always use Z for uniform scale
	const FVector RelativeDraggedTo = DraggingData.PassDraggedTo - DraggingData.GizmoStartTransform.GetLocation();
	const FVector AddedScaleOnAxis(RelativeDraggedTo.Z * VI::ScaleSensitivity->GetFloat());
	const FVector NewScale = DraggingData.GizmoStartTransform.GetScale3D() + AddedScaleOnAxis;
	DraggingData.OutGizmoUnsnappedTargetTransform.SetScale3D(NewScale);

	DraggingData.bOutMovedTransformGizmo = true;
	DraggingData.bOutShouldApplyVelocitiesFromDrag = true;
	DraggingData.bOutScaled = true;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
