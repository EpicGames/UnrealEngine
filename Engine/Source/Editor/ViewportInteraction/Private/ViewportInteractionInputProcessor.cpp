// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractionInputProcessor.h"
#include "Input/Events.h"
#include "Misc/App.h"
#include "ViewportWorldInteraction.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


FViewportInteractionInputProcessor::FViewportInteractionInputProcessor(UViewportWorldInteraction* InWorldInteraction)
	: WorldInteraction(InWorldInteraction)
{
}

FViewportInteractionInputProcessor::~FViewportInteractionInputProcessor()
{
	WorldInteraction = nullptr;
}

void FViewportInteractionInputProcessor::Tick( const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor )
{
}

bool FViewportInteractionInputProcessor::HandleKeyDownEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent )
{
	return WorldInteraction != nullptr ? WorldInteraction->PreprocessedInputKey( InKeyEvent.GetKey(), InKeyEvent.IsRepeat() ? IE_Repeat : IE_Pressed ) : false;
}

bool FViewportInteractionInputProcessor::HandleKeyUpEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent )
{
	return WorldInteraction != nullptr ? WorldInteraction->PreprocessedInputKey( InKeyEvent.GetKey(), IE_Released ) : false;
}

bool FViewportInteractionInputProcessor::HandleAnalogInputEvent( FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent )
{
	return WorldInteraction != nullptr ? WorldInteraction->PreprocessedInputAxis( InAnalogInputEvent.GetUserIndex(), InAnalogInputEvent.GetKey(), InAnalogInputEvent.GetAnalogValue(), FApp::GetDeltaTime() ) : false;
}

bool FViewportInteractionInputProcessor::HandleMouseMoveEvent( FSlateApplication& SlateApp, const FPointerEvent& MouseEvent )
{
	return false;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
