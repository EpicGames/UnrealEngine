// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/ICursor.h"
#include "Framework/Application/IInputProcessor.h"


class FSlateApplication;
class FViewportWorldInteractionManager;
struct FAnalogInputEvent;
struct FKeyEvent;
struct FPointerEvent;


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class FViewportInteractionInputProcessor : public IInputProcessor
{
public:
	FViewportInteractionInputProcessor(class UViewportWorldInteraction* InWorldInteraction);
	virtual ~FViewportInteractionInputProcessor ();

	//~ IInputProcess overrides
	virtual void Tick( const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor ) override;
	virtual bool HandleKeyDownEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent ) override;
	virtual bool HandleKeyUpEvent( FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent ) override;
	virtual bool HandleAnalogInputEvent( FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent ) override;
	virtual bool HandleMouseMoveEvent( FSlateApplication& SlateApp, const FPointerEvent& MouseEvent ) override;
	virtual const TCHAR* GetDebugName() const override { return TEXT("ViewportInteraction"); }

private:

	/** The WorldInteraction that will receive the input */
	class UViewportWorldInteraction* WorldInteraction;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
