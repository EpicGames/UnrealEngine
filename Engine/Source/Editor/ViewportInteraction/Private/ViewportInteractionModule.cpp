// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractionModule.h"
#include "HAL/IConsoleManager.h"
#include "EditorWorldExtension.h"
#include "ViewportWorldInteraction.h"
#include "Editor.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "SLevelViewport.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VI
{
	static FAutoConsoleCommand ForceMode(TEXT("VI.ForceMode"), TEXT("Toggles viewport interaction on desktop"), FConsoleCommandDelegate::CreateStatic(&FViewportInteractionModule::ToggleMode));
}

FViewportInteractionModule::FViewportInteractionModule() :
	bEnabledViewportWorldInteractionFromCommand(false)
{
}

FViewportInteractionModule::~FViewportInteractionModule()
{
}

IMPLEMENT_MODULE( FViewportInteractionModule, ViewportInteraction )

void FViewportInteractionModule::StartupModule()
{
}

void FViewportInteractionModule::ShutdownModule()
{
}

void FViewportInteractionModule::PostLoadCallback()
{
}


void FViewportInteractionModule::EnabledViewportWorldInteractionFromCommand(const bool bEnabled)
{
	bEnabledViewportWorldInteractionFromCommand = bEnabled;
}

bool FViewportInteractionModule::EnabledViewportWorldInteractionFromCommand()
{
	return bEnabledViewportWorldInteractionFromCommand;
}

void FViewportInteractionModule::ToggleMode()
{
	if (GEditor != nullptr)
	{
		//@todo ViewportInteraction: Get the world from the current viewport.
		UWorld* World = GEditor->bIsSimulatingInEditor ? GEditor->PlayWorld : GWorld;
		check (World != nullptr);
		
		UEditorWorldExtensionCollection* ExtensionCollection = GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(World);
		check (ExtensionCollection != nullptr);

		UViewportWorldInteraction* ViewportWorldInteraction = Cast<UViewportWorldInteraction>(ExtensionCollection->FindExtension(UViewportWorldInteraction::StaticClass()));

		FViewportInteractionModule& Self = FModuleManager::GetModuleChecked< FViewportInteractionModule >(TEXT("ViewportInteraction"));

		if (ViewportWorldInteraction == nullptr)
		{
			// There is no ViewportWorldInteraction, so we can create one and add it.
			UViewportWorldInteraction* NewViewportWorldInteraction = NewObject<UViewportWorldInteraction>(ExtensionCollection);
			ExtensionCollection->AddExtension(NewViewportWorldInteraction);
			NewViewportWorldInteraction->SetUseInputPreprocessor(true);

			// Set the current viewport.
			{
				const TSharedRef< ILevelEditor >& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor").GetFirstLevelEditor().ToSharedRef();

				// Do we have an active perspective viewport that is valid for VR?  If so, go ahead and use that.
 				TSharedPtr<FEditorViewportClient> ViewportClient;
				{
					TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
					if (ActiveLevelViewport.IsValid())
					{
						ViewportClient = StaticCastSharedRef<SLevelViewport>(ActiveLevelViewport->AsWidget())->GetViewportClient();
					}
				}

				NewViewportWorldInteraction->SetDefaultOptionalViewportClient(ViewportClient);
			}

			Self.EnabledViewportWorldInteractionFromCommand(true);
		}
		else if (Self.EnabledViewportWorldInteractionFromCommand())
		{
			// Close ViewportWorldInteraction, but only if we also started it with this command.
			ExtensionCollection->RemoveExtension(ViewportWorldInteraction);
			Self.EnabledViewportWorldInteractionFromCommand(false);
		}
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
