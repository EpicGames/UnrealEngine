// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IViewportInteractionModule.h"

class FViewportInteractionModule
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: public IViewportInteractionModule
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	
	FViewportInteractionModule();
	virtual ~FViewportInteractionModule();

	// FModuleInterface overrides
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual void PostLoadCallback() override;
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}
	
	static void ToggleMode();

	void EnabledViewportWorldInteractionFromCommand(const bool bEnabled);
	bool EnabledViewportWorldInteractionFromCommand();

private:

	/** If we started the ViewportWorldInteraction from Toggle command. */
	bool bEnabledViewportWorldInteractionFromCommand;
};
