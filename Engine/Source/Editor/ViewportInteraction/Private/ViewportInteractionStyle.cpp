// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractionStyle.h"

#include "Containers/UnrealString.h"
#include "Framework/Application/SlateApplication.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/SlateStyleRegistry.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class ISlateStyle;

TSharedPtr< FSlateStyleSet > FViewportInteractionStyle::ViewportInteractionStyleInstance = NULL;

void FViewportInteractionStyle::Initialize()
{
	if ( !ViewportInteractionStyleInstance.IsValid() )
	{
		ViewportInteractionStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle( *ViewportInteractionStyleInstance);
	}
}

void FViewportInteractionStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle( *ViewportInteractionStyleInstance);
	ensure(ViewportInteractionStyleInstance.IsUnique() );
	ViewportInteractionStyleInstance.Reset();
}

FName FViewportInteractionStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("ViewportInteractionStyle"));
	return StyleSetName;
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )

const FVector2D Icon14x14(14.0f, 14.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon64x64(64.0f, 64.0f);
const FVector2D Icon512x512(512.0f, 512.0f);

TSharedRef< FSlateStyleSet > FViewportInteractionStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet(FViewportInteractionStyle::GetStyleSetName()));

	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	return Style;
}

void FViewportInteractionStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FViewportInteractionStyle::Get()
{
	return *ViewportInteractionStyleInstance;
}


#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH


PRAGMA_ENABLE_DEPRECATION_WARNINGS
