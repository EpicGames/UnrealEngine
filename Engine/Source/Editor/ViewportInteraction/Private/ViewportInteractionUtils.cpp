// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractionUtils.h"

#include "Math/UnrealMathUtility.h"

namespace ViewportInteractionUtils
{
	
	FOneEuroFilter::FLowpassFilter::FLowpassFilter():
		Previous(FVector::ZeroVector),
		bFirstTime(true)
	{

	}

	FVector FOneEuroFilter::FLowpassFilter::Filter(const FVector& InValue, const FVector& InAlpha)
	{
		FVector Result = InValue;
		if (!bFirstTime)
		{
			for (int i = 0; i < 3; i++)
			{
				Result[i] = InAlpha[i] * InValue[i] + (1 - InAlpha[i]) * Previous[i];
			}
		}

		bFirstTime = false;
		Previous = Result;
		return Result;
	}

	bool FOneEuroFilter::FLowpassFilter::IsFirstTime() const
	{
		return bFirstTime;
	}

	FVector FOneEuroFilter::FLowpassFilter::GetPrevious() const
	{
		return Previous;
	}

	FOneEuroFilter::FOneEuroFilter() :
		MinCutoff(1.0f),
		CutoffSlope(0.007f),
		DeltaCutoff(1.0f)
	{

	}

	FOneEuroFilter::FOneEuroFilter(const double InMinCutoff, const double InCutoffSlope, const double InDeltaCutoff) :
		MinCutoff(InMinCutoff),
		CutoffSlope(InCutoffSlope),
		DeltaCutoff(InDeltaCutoff)
	{

	}

	FVector FOneEuroFilter::Filter(const FVector& InRaw, const double InDeltaTime)
	{
		// Calculate the delta, if this is the first time then there is no delta
		const FVector Delta = RawFilter.IsFirstTime() == true ? FVector::ZeroVector : (InRaw - RawFilter.GetPrevious()) * InDeltaTime;

		// Filter the delta to get the estimated
		const FVector Estimated = DeltaFilter.Filter(Delta, FVector(CalculateAlpha(DeltaCutoff, InDeltaTime)));

		// Use the estimated to calculate the cutoff
		const FVector Cutoff = CalculateCutoff(Estimated);

		// Filter passed value 
		return RawFilter.Filter(InRaw, CalculateAlpha(Cutoff, InDeltaTime));
	}

	void FOneEuroFilter::SetMinCutoff(const double InMinCutoff)
	{
		MinCutoff = InMinCutoff;
	}

	void FOneEuroFilter::SetCutoffSlope(const double InCutoffSlope)
	{
		CutoffSlope = InCutoffSlope;
	}

	void FOneEuroFilter::SetDeltaCutoff(const double InDeltaCutoff)
	{
		DeltaCutoff = InDeltaCutoff;
	}

	const FVector FOneEuroFilter::CalculateCutoff(const FVector& InValue)
	{
		FVector Result;
		for (int i = 0; i < 3; i++)
		{
			Result[i] = MinCutoff + CutoffSlope * FMath::Abs(InValue[i]);
		}
		return Result;
	}

	const FVector FOneEuroFilter::CalculateAlpha(const FVector& InCutoff, const double InDeltaTime) const
	{
		FVector Result;
		for (int i = 0; i < 3; i++)
		{
			Result[i] = CalculateAlpha(InCutoff[i], InDeltaTime);
		}
		return Result;
	}

	const double FOneEuroFilter::CalculateAlpha(const double InCutoff, const double InDeltaTime) const
	{
		const double tau = 1.0 / (2 * UE_DOUBLE_PI * InCutoff);
		return 1.0 / (1.0 + tau / InDeltaTime);
	}

}