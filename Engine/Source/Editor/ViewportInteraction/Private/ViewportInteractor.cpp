// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractor.h"
#include "CollisionQueryParams.h"
#include "Components/PrimitiveComponent.h"
#include "Editor.h"
#include "ViewportWorldInteraction.h"
#include "ViewportInteractableInterface.h"
#include "MouseCursorInteractor.h"
#include "ScopedTransaction.h"
#include "DrawDebugHelpers.h"
#include "EngineUtils.h"
#include "ActorTransformer.h"
#include "Engine/Selection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportInteractor)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "ViewportInteractor"

namespace VI
{
	static FAutoConsoleVariable GrabberSphereRadius( TEXT( "VI.GrabberSphereRadius" ), 12.0f, TEXT( "In radial mode, the radius of the sphere used to select and interact" ) );
	static FAutoConsoleVariable GrabberSphereOffset( TEXT( "VI.GrabberSphereOffset" ), 2.0f, TEXT( "Offset from the controller origin that the grabber sphere should be centered at" ) );
	static FAutoConsoleVariable LaserPointerMaxLength( TEXT( "VI.LaserPointerMaxLength" ), 30000.0f, TEXT( "Maximum length of the laser pointer line" ) );
	static FAutoConsoleVariable DragHapticFeedbackStrength( TEXT( "VI.DragHapticFeedbackStrength" ), 1.0f, TEXT( "Default strength for haptic feedback when starting to drag objects" ) ); //@todo ViewportInteraction: Duplicate from ViewportWorldInteraction
	static FAutoConsoleVariable SelectionHapticFeedbackStrength( TEXT( "VI.SelectionHapticFeedbackStrength" ), 0.5f, TEXT( "Default strength for haptic feedback when selecting objects" ) );
	static FAutoConsoleVariable LaserPointerRetractDuration( TEXT( "VI.LaserPointerRetractDuration" ), 0.2f, TEXT( "How fast the laser pointer should extend or retract" ) );
	static FAutoConsoleVariable AllowLaserSmooth(TEXT("VI.AllowLaserSmooth"), 1, TEXT("Allow laser smoothing using one euro"));
	static FAutoConsoleVariable LaserSmoothLag(TEXT("VI.LaserSmoothLag"), 0.007f, TEXT("Laser smooth lag"));
	static FAutoConsoleVariable LaserSmoothMinimumCutoff(TEXT("VI.LaserSmoothMinimumCutoff"), 0.9f, TEXT("Laser smooth lag"));
}

UViewportInteractor::UViewportInteractor() :
	Super(),
	InteractorData(),
	KeyToActionMap(),
	WorldInteraction( nullptr ),
	OtherInteractor( nullptr ),
	bAllowGrabberSphere( true ),
	bCanCarry( false ),
	SavedLaserPointerEnd()
{
	CurrentHitResultGizmoFilterMode = EHitResultGizmoFilterMode::All;
	SmoothingOneEuroFilter = ViewportInteractionUtils::FOneEuroFilter(VI::LaserSmoothMinimumCutoff->GetFloat(), VI::LaserSmoothLag->GetFloat(), 1.0f);
}

FViewportInteractorData& UViewportInteractor::GetInteractorData()
{
	return InteractorData;
}

void UViewportInteractor::SetWorldInteraction( UViewportWorldInteraction* InWorldInteraction )
{
	WorldInteraction = InWorldInteraction;
}

UViewportWorldInteraction* UViewportInteractor::GetWorldInteraction() const
{
	return WorldInteraction;
}

void UViewportInteractor::SetOtherInteractor( UViewportInteractor* InOtherInteractor )
{
	OtherInteractor = InOtherInteractor;
}

void UViewportInteractor::RemoveOtherInteractor()
{
	OtherInteractor = nullptr;
}

UViewportInteractor* UViewportInteractor::GetOtherInteractor() const
{
	return OtherInteractor;
}

void UViewportInteractor::Shutdown_Implementation()
{
	KeyToActionMap.Empty();

	WorldInteraction = nullptr;
	OtherInteractor = nullptr;
}

void UViewportInteractor::Tick_Implementation( const float DeltaTime )
{
}

void UViewportInteractor::OnStartDragging( const FVector& HitLocation, const bool bIsPlacingNewObjects )
{
	// If the user is holding down the modifier key, go ahead and duplicate the selection first.  Don't do this if we're
	// placing objects right now though.
	if( IsModifierPressed() && !bIsPlacingNewObjects )
	{
		// Only duplicate selected objects if we're using the "actor" transformer.  Other transformable types might not support duplication!
		if( Cast<const UActorTransformer>( WorldInteraction->GetTransformer() ) != nullptr )
		{
			WorldInteraction->Duplicate();
		}
	}
}

class UActorComponent* UViewportInteractor::GetLastHoverComponent()
{
	return InteractorData.LastHoveredActorComponent.Get();
}

void UViewportInteractor::AddKeyAction( const FKey& Key, const FViewportActionKeyInput& Action )
{
	KeyToActionMap.Add( Key, Action );
}

void UViewportInteractor::RemoveKeyAction( const FKey& Key )
{
	KeyToActionMap.Remove( Key );
}

bool UViewportInteractor::HandleInputKey( FEditorViewportClient& ViewportClient, const FKey Key, const EInputEvent Event )
{
	bool bHandled = false;
	FViewportActionKeyInput* Action = KeyToActionMap.Find( Key );
	if ( Action != nullptr )	// Ignore key repeats
	{
		if ( Action->bIsAxis )
		{
			UE_LOGF(LogInput, Warning, "UViewportInteractor: %ls registered as axis but handled as key", *Action->ActionType.ToString());
			Action->bIsAxis = false;
		}

		Action->Event = Event;

		if( !bHandled )
		{
			// Give the derived classes a chance to update according to the input
			PreviewInputKey( ViewportClient, *Action, Key, Event, bHandled );
		}

		if (!bHandled)
		{
			// Here we try to call an BP overridden function.  This is cut down to not include non BP friendly classes.
			HandleInputKey_BP( *Action, Key, Event, bHandled );
		}

		FHitResult HitResult = GetHitResultFromLaserPointer(nullptr, CurrentHitResultGizmoFilterMode);

		if( !bHandled )
		{
			// "Preview" the event first.  This gives a first chance for systems to intercept the event regardless of registration order.
			WorldInteraction->OnPreviewInputAction().Broadcast( ViewportClient, this, *Action, Action->bIsInputCaptured, bHandled );
		}

		if( !bHandled )
		{
			// Prefer transform gizmo interactions over anything else
			const bool bPressedTransformGizmo =
				Event == IE_Pressed &&
				HitResult.HitObjectHandle.IsValid() &&
				HitResult.HitObjectHandle == WorldInteraction->GetTransformGizmoActor() &&
				Action->ActionType == ViewportWorldActionTypes::SelectAndMove;
			if( !bPressedTransformGizmo )
			{
				// Give subsystems a chance to handle actions for this interactor
				WorldInteraction->OnViewportInteractionInputAction().Broadcast( ViewportClient, this, *Action, Action->bIsInputCaptured, bHandled );
			}
		}

		if(!bHandled)
		{
			// Give the derived classes a chance to update according to the input
			HandleInputKey( ViewportClient, *Action, Key, Event, bHandled );
		}

		// Start checking on default action implementation
		if ( !bHandled )
		{
			// Selection/Movement
			if( Action->ActionType == ViewportWorldActionTypes::SelectAndMove )
			{
				const bool bIsDraggingWorld = InteractorData.DraggingMode == EViewportInteractionDraggingMode::World;
				const bool bIsDraggingWorldWithTwoHands =
					OtherInteractor != nullptr &&
					( ( InteractorData.DraggingMode == EViewportInteractionDraggingMode::World && GetOtherInteractor()->GetInteractorData().DraggingMode == EViewportInteractionDraggingMode::AssistingDrag ) ||
					  ( GetOtherInteractor()->GetInteractorData().DraggingMode == EViewportInteractionDraggingMode::World && InteractorData.DraggingMode == EViewportInteractionDraggingMode::AssistingDrag ) );

				if ( Event == IE_Pressed )
				{
					// No clicking while we're dragging the world.  (No laser pointers are visible, anyway.)
					if ( !bIsDraggingWorldWithTwoHands && HitResult.HitObjectHandle.IsValid() )
					{
						if ( WorldInteraction->IsInteractableComponent( HitResult.GetComponent() ) )
						{
							InteractorData.ClickingOnComponent = HitResult.GetComponent();	// @todo gizmo: Should be changed to store only gizmo components?

							AActor* Actor = HitResult.HitObjectHandle.FetchActor();

							FVector LaserPointerStart, LaserPointerEnd;
							const bool bHaveLaserPointer = GetLaserPointer( /* Out */ LaserPointerStart, /* Out */ LaserPointerEnd );
							if (!bHaveLaserPointer)
							{
								return false;
							}

							bool bCanBeSelected = true;
							// This is a "manual" interface Cast<> to avoid emitting a spurious deprecation warning.
							// This code will all be deleted in UE 5.9.
							if (Actor->GetClass()->ImplementsInterface(UViewportInteractableInterface::StaticClass()))
							{
								IViewportInteractableInterface* ActorInteractable =
									(IViewportInteractableInterface*)Actor->GetInterfaceAddress(UViewportInteractableInterface::StaticClass());

								bCanBeSelected = ActorInteractable->CanBeSelected();
								if (!bCanBeSelected)
								{
									bHandled = true;
									bool bResultedInInteractableDrag = false;
									ActorInteractable->OnPressed(this, HitResult, bResultedInInteractableDrag);

									if (bResultedInInteractableDrag)
									{
										WorldInteraction->SetDraggedInteractable(ActorInteractable, this);
									}
								}
							}

							if (bCanBeSelected)
							{
								bHandled = true;

								FViewportInteractorData* OtherInteractorData = nullptr;
								if( OtherInteractor != nullptr )
								{
									OtherInteractorData = &OtherInteractor->GetInteractorData();
								}

								// Is the other hand already dragging this stuff?
								if ( OtherInteractorData != nullptr &&
									 ( OtherInteractorData->DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
									   OtherInteractorData->DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ) )
								{
									// Only if they clicked on one of the objects we're already moving
									if ( Actor->IsSelected() )	// @todo gizmo: Should check for existing transformable, not an actor!
									{
										// If we were dragging with a gizmo, we'll need to stop doing that and instead drag freely.
										if ( OtherInteractorData->DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo )
										{
											WorldInteraction->StopDragging( this );

											UPrimitiveComponent* ClickedTransformGizmoComponent = nullptr;
											const bool bIsPlacingNewObjects = false;
											const bool bAllowInterpolationWhenPlacing = false;
											const bool bShouldUseLaserImpactDrag = false;
											const bool bStartTransaction = !WorldInteraction->GetTrackingTransaction().IsActive();
											const bool bWithGrabberSphere = false;	// @todo grabber: Not supported yet
											WorldInteraction->StartDragging( OtherInteractor, ClickedTransformGizmoComponent, OtherInteractor->GetHoverLocation(), bIsPlacingNewObjects, bAllowInterpolationWhenPlacing, bShouldUseLaserImpactDrag, bStartTransaction, bWithGrabberSphere );
										}

										InteractorData.DraggingMode = InteractorData.LastDraggingMode = EViewportInteractionDraggingMode::AssistingDrag;

										InteractorData.bDraggingWithGrabberSphere = false;	// @todo grabber: Not supported yet
										InteractorData.bIsFirstDragUpdate = true;
										InteractorData.bWasAssistingDrag = true;
										InteractorData.DragRayLength = ( HitResult.ImpactPoint - LaserPointerStart ).Size();
										InteractorData.LastDragToLocation = HitResult.ImpactPoint;
										InteractorData.InteractorTransformAtDragStart = InteractorData.Transform;
										InteractorData.GrabberSphereLocationAtDragStart = FVector::ZeroVector;
										InteractorData.ImpactLocationAtDragStart = HitResult.ImpactPoint;
										InteractorData.DragTranslationVelocity = FVector::ZeroVector;
										InteractorData.DragRayLengthVelocity = 0.0f;
										InteractorData.DraggingTransformGizmoComponent = nullptr;
										InteractorData.DragOperationComponent.Reset();
										InteractorData.bIsDrivingVelocityOfSimulatedTransformables = false;

										InteractorData.GizmoStartTransform = OtherInteractorData->GizmoStartTransform;
										InteractorData.GizmoLastTransform = InteractorData.GizmoTargetTransform = InteractorData.GizmoUnsnappedTargetTransform = InteractorData.GizmoInterpolationSnapshotTransform = InteractorData.GizmoStartTransform;
										InteractorData.GizmoStartLocalBounds = OtherInteractorData->GizmoStartLocalBounds;
										InteractorData.GizmoSpaceFirstDragUpdateOffsetAlongAxis = FVector::ZeroVector;	// Will be determined on first update
										InteractorData.LockedWorldDragMode = ELockedWorldDragMode::Unlocked;
										InteractorData.GizmoScaleSinceDragStarted = 0.0f;
										InteractorData.GizmoRotationRadiansSinceDragStarted = 0.0f;
										InteractorData.GizmoSpaceDragDeltaFromStartOffset = FVector::ZeroVector;	// Set every frame while dragging

										WorldInteraction->SetDraggedSinceLastSelection( true );
										WorldInteraction->SetLastDragGizmoStartTransform( InteractorData.GizmoStartTransform );

										Action->bIsInputCaptured = true;

										// Play a haptic effect when objects are picked up
										PlayHapticEffect( VI::DragHapticFeedbackStrength->GetFloat() ); //@todo ViewportInteraction
									}
									else
									{
										// @todo vreditor: We don't currently support selection/dragging separate objects with either hand
									}
								}
								else if ( OtherInteractorData != nullptr && OtherInteractorData->DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo )
								{
									// We don't support dragging objects with the gizmo using two hands.  Just absorb it.
								}
								else if ( OtherInteractorData != nullptr && OtherInteractorData->DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
								{
									// Doesn't work with two hands.  Just absorb it.
								}
								else
								{
									// Only start dragging if the actor was already selected (and it's a full press), or if we clicked on a gizmo.  It's too easy to
									// accidentally move actors around the scene when you only want to select them otherwise.
									const bool bOtherHandTryingToDrag =
										Actor != WorldInteraction->GetTransformGizmoActor() &&
										OtherInteractorData != nullptr &&
										OtherInteractorData->ClickingOnComponent.IsValid() &&
										OtherInteractorData->ClickingOnComponent.Get()->GetOwner()->IsSelected() &&
										OtherInteractorData->ClickingOnComponent.Get()->GetOwner() == HitResult.GetComponent()->GetOwner();	// Trying to drag same actor

									FTrackingTransaction& TrackingTransaction = WorldInteraction->GetTrackingTransaction();

									bool bShouldDragSelected = false;
									bool bSelectionChanged = false;
									if( Actor == WorldInteraction->GetTransformGizmoActor() )
									{
										bShouldDragSelected = true;
										bSelectionChanged = false;
									}
									else
									{
										// Clicked on a normal actor.  So update selection!
										const bool bWasSelected = Actor->IsSelected();

										// Determine whether we are starting a marquee select
										const bool bIsMarqueeSelect = ViewportClient.IsAltPressed() && ViewportClient.IsCtrlPressed();

										// Default to replacing our selection with whatever the user clicked on
										enum ESelectionModification
										{
											Replace,
											Toggle
										} SelectionModification = ESelectionModification::Replace;

										// Light pressing on an actor while holding the modifier button will toggle selection
										const bool bIsMouseCursorInteractor = this->IsA( UMouseCursorInteractor::StaticClass() );
										if( this->IsModifierPressed() && !bWasSelected )
										{
											SelectionModification = ESelectionModification::Toggle;
										}

										if( !bIsMarqueeSelect )
										{
											if( !bWasSelected && SelectionModification == ESelectionModification::Replace )
											{
												// Capture undo state
												bSelectionChanged = true;
												TrackingTransaction.TransCount++;
												TrackingTransaction.Begin( LOCTEXT( "SelectActor", "Select Actor" ) );

												GEditor->SelectNone( true, true );
												GEditor->SelectActor( Actor, true, true );
											}
											else if ( SelectionModification == ESelectionModification::Toggle )
											{
												TrackingTransaction.TransCount++;
												TrackingTransaction.Begin( LOCTEXT( "ToggleActorSelection", "Toggle Actor Selection" ) );

												GEditor->SelectActor( Actor, !Actor->IsSelected(), true );
												bSelectionChanged = true;
											}
										}

										bShouldDragSelected = bOtherHandTryingToDrag || !bSelectionChanged;

										// Carryable items, like cameras are a special case, we want to be able to click and move in one action.
										const TArray< TUniquePtr< FViewportTransformable > >&Transformables = WorldInteraction->GetTransformables();
										if (!bShouldDragSelected && Transformables.Num() == 1 && Transformables[0].Get()->ShouldBeCarried())
										{
											bShouldDragSelected = true;
										}
									}

									if ( bShouldDragSelected && InteractorData.DraggingMode != EViewportInteractionDraggingMode::Interactable )
									{
										UPrimitiveComponent* ClickedTransformGizmoComponent = nullptr;
										{
											const bool bUsingGizmo =
												HitResult.GetComponent() != nullptr &&
												HitResult.GetComponent()->GetOwner() == WorldInteraction->GetTransformGizmoActor();
											if( bUsingGizmo )
											{
												ClickedTransformGizmoComponent = HitResult.GetComponent();
											}
										}

										const bool bIsPlacingNewObjects = false;
										const bool bAllowInterpolationWhenPlacing = true;
										const bool bShouldUseLaserImpactDrag = false;
										const bool bStartTransaction = !bSelectionChanged;
										const bool bWithGrabberSphere = false;	// @todo grabber: Not supported yet
										WorldInteraction->StartDragging( this, ClickedTransformGizmoComponent, HitResult.ImpactPoint, bIsPlacingNewObjects, bAllowInterpolationWhenPlacing, bShouldUseLaserImpactDrag, bStartTransaction, bWithGrabberSphere );

										Action->bIsInputCaptured = true;
									}
									else if ( bSelectionChanged )
									{
										// Stop our transaction
										TrackingTransaction.End();

										// User didn't drag but did select something, so play a haptic effect.
										PlayHapticEffect( VI::SelectionHapticFeedbackStrength->GetFloat() );
									}
								}
							}
						}
					}
				}
				else if ( Event == IE_Released )
				{
					if ( InteractorData.ClickingOnComponent.IsValid() )
					{
						bHandled = true;
						InteractorData.ClickingOnComponent = nullptr;
					}

					// Don't allow the trigger to cancel our drag on release if we're dragging the world.
					if ( InteractorData.DraggingMode != EViewportInteractionDraggingMode::Nothing &&
						!bIsDraggingWorld &&
						!bIsDraggingWorldWithTwoHands )
					{
						WorldInteraction->StopDragging( this );
						Action->bIsInputCaptured = false;
						bHandled = true;
					}
				}
			}
			// World Movement
			else if ( Action->ActionType == ViewportWorldActionTypes::WorldMovement )
			{
				if ( Event == IE_Pressed )
				{
					// Is our other hand already dragging the world around?
					if ( OtherInteractor && OtherInteractor->GetInteractorData().DraggingMode == EViewportInteractionDraggingMode::World )
					{
						InteractorData.DraggingMode = InteractorData.LastDraggingMode = EViewportInteractionDraggingMode::AssistingDrag;
						InteractorData.bWasAssistingDrag = true;
					}
					else
					{
						// Start dragging the world
						InteractorData.DraggingMode = InteractorData.LastDraggingMode = EViewportInteractionDraggingMode::World;
						InteractorData.bWasAssistingDrag = false;

						if( OtherInteractor != nullptr )
						{
							// Starting a new drag, so make sure the other hand doesn't think it's assisting us
							OtherInteractor->GetInteractorData().bWasAssistingDrag = false;

							// Stop any interia from the other hand's previous movements -- we've grabbed the world and it needs to stick!
							OtherInteractor->GetInteractorData().DragTranslationVelocity = FVector::ZeroVector;
						}
					}

					InteractorData.bIsFirstDragUpdate = true;
					InteractorData.bDraggingWithGrabberSphere = false;
					InteractorData.DragRayLength = 0.0f;
					InteractorData.LastDragToLocation = InteractorData.Transform.GetLocation();
					InteractorData.InteractorTransformAtDragStart = InteractorData.Transform;
					InteractorData.GrabberSphereLocationAtDragStart = FVector::ZeroVector;
					InteractorData.DragTranslationVelocity = FVector::ZeroVector;
					InteractorData.DragRayLengthVelocity = 0.0f;
					InteractorData.bIsDrivingVelocityOfSimulatedTransformables = false;

					// We won't use gizmo features for world movement
					InteractorData.DraggingTransformGizmoComponent = nullptr;
					InteractorData.DragOperationComponent.Reset();
					InteractorData.OptionalHandlePlacement.Reset();
					InteractorData.GizmoStartTransform = FTransform::Identity;
					InteractorData.GizmoStartLocalBounds = FBox(ForceInit);
					InteractorData.GizmoLastTransform = InteractorData.GizmoTargetTransform = InteractorData.GizmoUnsnappedTargetTransform = InteractorData.GizmoInterpolationSnapshotTransform = InteractorData.GizmoStartTransform;
					InteractorData.LockedWorldDragMode = ELockedWorldDragMode::Unlocked;
					InteractorData.GizmoScaleSinceDragStarted = 0.0f;
					InteractorData.GizmoRotationRadiansSinceDragStarted = 0.0f;
					InteractorData.GizmoSpaceFirstDragUpdateOffsetAlongAxis = FVector::ZeroVector;
					InteractorData.GizmoSpaceDragDeltaFromStartOffset = FVector::ZeroVector;

					Action->bIsInputCaptured = true;
				}
				else if ( Event == IE_Released )
				{
					WorldInteraction->StopDragging( this );

					Action->bIsInputCaptured = false;
				}
			}
			else if ( Action->ActionType == ViewportWorldActionTypes::Delete )
			{
				if ( Event == IE_Pressed )
				{
					WorldInteraction->DeleteSelectedObjects();
				}
				bHandled = true;
			}
			else if ( Action->ActionType == ViewportWorldActionTypes::Redo )
			{
				if ( Event == IE_Pressed || Event == IE_Repeat )
				{
					WorldInteraction->Redo();
				}
				bHandled = true;
			}
			else if ( Action->ActionType == ViewportWorldActionTypes::Undo )
			{
				if ( Event == IE_Pressed || Event == IE_Repeat )
				{
					WorldInteraction->Undo();
				}
				bHandled = true;
			}
		}

		if ( !bHandled )
		{
			// Determine whether we are starting a marquee select
			const bool bIsMarqueeSelect = ViewportClient.IsAltPressed() && ViewportClient.IsCtrlPressed();

			// If "select and move" was pressed but not handled, go ahead and deselect everything
			if ( !bIsMarqueeSelect && Action->ActionType == ViewportWorldActionTypes::SelectAndMove && Event == IE_Pressed )
			{
				WorldInteraction->Deselect();
			}

			WorldInteraction->OnViewportInteractionInputUnhandled().Broadcast( ViewportClient, this, *Action );
		}
	}

	return bHandled;
}

bool UViewportInteractor::HandleInputAxis( FEditorViewportClient& ViewportClient, const FKey Key, const float Delta, const float DeltaTime )
{
	bool bHandled = false;
	FViewportActionKeyInput* KnownAction = KeyToActionMap.Find( Key );
	if ( KnownAction != nullptr )	// Ignore key repeats
	{
		if (!KnownAction->bIsAxis)
		{
			UE_LOGF(LogInput, Warning, "UViewportInteractor: %ls registered as key but handled as axis", *KnownAction->ActionType.ToString());
			KnownAction->bIsAxis = true;
		}

		FViewportActionKeyInput Action( KnownAction->ActionType );

		if( !bHandled )
		{
			// Give the derived classes a chance to update according to the input
			PreviewInputAxis( ViewportClient, Action, Key, Delta, DeltaTime, bHandled );
		}

		if (!bHandled)
		{
			// Here we try to call an BP overridden function.  This is cut down to not include non BP friendly classes.
			HandleInputAxis_BP( Action, Key, Delta, DeltaTime, bHandled );
		}

		if( !bHandled )
		{
			// Give the derived classes a chance to update according to the input
			HandleInputAxis( ViewportClient, Action, Key, Delta, DeltaTime, bHandled );
		}
	}

	return bHandled;
}

bool UViewportInteractor::AllowLaserSmoothing() const
{
	return true;
}

FTransform UViewportInteractor::GetTransform() const
{
	return InteractorData.Transform;
}

FTransform UViewportInteractor::GetRoomSpaceTransform() const
{
	return InteractorData.RoomSpaceTransform;
}

FTransform UViewportInteractor::GetLastTransform() const
{
	return InteractorData.LastTransform;
}

FTransform UViewportInteractor::GetLastRoomSpaceTransform() const
{
	return InteractorData.LastRoomSpaceTransform;
}

EViewportInteractionDraggingMode UViewportInteractor::GetDraggingMode() const
{
	return InteractorData.DraggingMode;
}

EViewportInteractionDraggingMode UViewportInteractor::GetLastDraggingMode() const
{
	return InteractorData.LastDraggingMode;
}

FVector UViewportInteractor::GetDragTranslationVelocity() const
{
	return InteractorData.DragTranslationVelocity;
}

void UViewportInteractor::SetHoverLocation(const FVector& InHoverLocation)
{
	InteractorData.HoverLocation = InHoverLocation;
}

bool UViewportInteractor::GetLaserPointer( FVector& LaserPointerStart, FVector& LaserPointerEnd, const bool bEvenIfBlocked, const float LaserLengthOverride )
{
	// If we have UI attached to us, don't allow a laser pointer
	if( bEvenIfBlocked || !GetIsLaserBlocked() )
	{
		FTransform HandTransform;
		FVector HandForwardVector;
		if (GetTransformAndForwardVector(HandTransform, HandForwardVector))
		{
			LaserPointerStart = HandTransform.GetLocation();

			/** To avoid calculating the smooth end location multiple times in one tick we will check if it has already been done this frame and use that */
			if (!SavedLaserPointerEnd.IsSet())
			{
				const float LaserLength = LaserLengthOverride == 0.0f ? GetLaserPointerMaxLength() : LaserLengthOverride;
				FVector FinalLaserPointerEnd = LaserPointerStart + HandForwardVector * LaserLength;

				/** Only smooth the endlocation when not disabled */
				if (WorldInteraction != nullptr && AllowLaserSmoothing() && VI::AllowLaserSmooth->GetInt() == 1)
				{
					SmoothingOneEuroFilter.SetCutoffSlope(VI::LaserSmoothLag->GetFloat());
					SmoothingOneEuroFilter.SetMinCutoff(VI::LaserSmoothMinimumCutoff->GetFloat());
					FinalLaserPointerEnd = SmoothingOneEuroFilter.Filter(FinalLaserPointerEnd, WorldInteraction->GetCurrentDeltaTime());
				}

				LaserPointerEnd = FinalLaserPointerEnd;
				SavedLaserPointerEnd = FinalLaserPointerEnd;
			}
			else
			{
				LaserPointerEnd = SavedLaserPointerEnd.GetValue();
			}

			return true;
		}
	}
	return false;
}


bool UViewportInteractor::GetGrabberSphere( FSphere& OutGrabberSphere, const bool bEvenIfBlocked )
{
	OutGrabberSphere = FSphere( 0 );

	if( bAllowGrabberSphere )
	{
		FVector LaserPointerStart, LaserPointerEnd;
		if( GetLaserPointer( LaserPointerStart, LaserPointerEnd, bEvenIfBlocked ) )
		{
			FTransform HandTransform;
			FVector HandForwardVector;
			if( GetTransformAndForwardVector( HandTransform, HandForwardVector ) )
			{
				const FVector GrabberSphereCenter = HandTransform.GetLocation() + HandForwardVector * VI::GrabberSphereOffset->GetFloat() * WorldInteraction->GetWorldScaleFactor();

				OutGrabberSphere = FSphere( GrabberSphereCenter, VI::GrabberSphereRadius->GetFloat() * WorldInteraction->GetWorldScaleFactor() );
				return true;
			}
		}
	}

	return false;
}


float UViewportInteractor::GetLaserPointerMaxLength() const
{
	return VI::LaserPointerMaxLength->GetFloat();
}

void UViewportInteractor::ResetHoverState()
{
	InteractorData.HoverLocation.Reset();
	InteractorData.HoveringOverTransformGizmoComponent = nullptr;
	SavedHitResult.Reset();
}

FHitResult UViewportInteractor::GetHitResultFromLaserPointer( TArray<AActor*>* OptionalListOfIgnoredActors /*= nullptr*/, const EHitResultGizmoFilterMode GizmoFilterMode /*= EHitResultGizmoFilterMode::All*/,
	TArray<UClass*>* ObjectsInFrontOfGizmo /*= nullptr */, const bool bEvenIfBlocked /*= false */, const float LaserLengthOverride /*= 0.0f */ )
{
	FHitResult BestHitResult;

	if (SavedHitResult.IsSet() && OptionalListOfIgnoredActors != nullptr && !OptionalListOfIgnoredActors->Contains(SavedHitResult->HitObjectHandle.FetchActor()) && (SavedHitResultFilterMode.IsSet() && (GizmoFilterMode == SavedHitResultFilterMode.GetValue())))
	{
		BestHitResult = SavedHitResult.GetValue();
	}
	else
	{
		FVector LaserPointerStart, LaserPointerEnd;
		if ( GetLaserPointer( LaserPointerStart, LaserPointerEnd, bEvenIfBlocked, LaserLengthOverride ) )
		{
			bool bActuallyIgnoreGizmos = (GizmoFilterMode == EHitResultGizmoFilterMode::NoGizmos);
			if( !WorldInteraction->IsTransformGizmoVisible() )
			{
				bActuallyIgnoreGizmos = true;
			}

			// Ignore all volume objects.  They'll just get in the way of selecting other things.
			// @todo viewportinteraction: We'll need to device a way to allow volume wire bounds to be selectable using this system
			static TArray<AActor*> VolumeActors;
			VolumeActors.Reset();
			if (WorldInteraction->GetWorld() != nullptr)
			{
				for( TActorIterator<AVolume> It( WorldInteraction->GetWorld(), AVolume::StaticClass() ); It; ++It )
				{
					AActor* Actor = *It;
					if( IsValid(Actor) )
					{
						VolumeActors.Add( Actor );
					}
				}
			}

			// Twice twice.  Once for editor gizmos which are "on top" and always take precedence, then a second time
			// for all of the scene objects
			for ( int32 PassIndex = bActuallyIgnoreGizmos ? 1 : 0; PassIndex < ((GizmoFilterMode == EHitResultGizmoFilterMode::GizmosOnly) ? 1 : 2); ++PassIndex )
			{
				const bool bOnlyEditorGizmos = ( PassIndex == 0 );

				const bool bTraceComplex = true;
				FCollisionQueryParams TraceParams( NAME_None, FCollisionQueryParams::GetUnknownStatId(), bTraceComplex, nullptr );

				if ( OptionalListOfIgnoredActors != nullptr )
				{
					TraceParams.AddIgnoredActors( *OptionalListOfIgnoredActors );
				}

				for( const TWeakObjectPtr<AActor>& ActorToIgnoreWeakPtr : WorldInteraction->GetActorsToExcludeFromHitTest() )
				{
					AActor* ActorToIgnore = ActorToIgnoreWeakPtr.Get();
					if( ActorToIgnore != nullptr )
					{
						TraceParams.AddIgnoredActor( ActorToIgnore );
					}
				}

				TraceParams.AddIgnoredActors( VolumeActors );

				bool bHit = false;
				FHitResult HitResult;
				if ( bOnlyEditorGizmos )
				{
					const FCollisionResponseParams& ResponseParam = FCollisionResponseParams::DefaultResponseParam;
					const ECollisionChannel CollisionChannel = bOnlyEditorGizmos ? COLLISION_GIZMO : ECC_Visibility;

					bHit = WorldInteraction->GetWorld()->LineTraceSingleByChannel( HitResult, LaserPointerStart, LaserPointerEnd, CollisionChannel, TraceParams, ResponseParam );
					if ( bHit )
					{
						BestHitResult = HitResult;
					}
				}
				else
				{
					FCollisionObjectQueryParams EverythingButGizmos( FCollisionObjectQueryParams::AllObjects );
					EverythingButGizmos.RemoveObjectTypesToQuery( COLLISION_GIZMO );
					bHit = WorldInteraction->GetWorld()->LineTraceSingleByObjectType( HitResult, LaserPointerStart, LaserPointerEnd, EverythingButGizmos, TraceParams );

					if ( bHit )
					{
						InteractorData.bHitResultIsPriorityType = false;
						if ( !bOnlyEditorGizmos && ObjectsInFrontOfGizmo )
						{
							for ( UClass* CurrentClass : *ObjectsInFrontOfGizmo )
							{
								bool bClassHasPriority = false;
								bClassHasPriority =
									( HitResult.GetComponent() != nullptr && HitResult.GetComponent()->IsA( CurrentClass ) ) ||
									( HitResult.HitObjectHandle.IsValid() && HitResult.HitObjectHandle.DoesRepresentClass( CurrentClass ) );

								if ( bClassHasPriority )
								{
									InteractorData.bHitResultIsPriorityType = bClassHasPriority;
									break;
								}
							}
						}

						const bool bHitResultIsGizmo = HitResult.HitObjectHandle.IsValid() && HitResult.HitObjectHandle == WorldInteraction->GetTransformGizmoActor();
						if ( !BestHitResult.HitObjectHandle.IsValid() ||
							 InteractorData.bHitResultIsPriorityType ||
							 bHitResultIsGizmo )
						{
							BestHitResult = HitResult;
						}
					}
				}
			}
		}

		SavedHitResult = BestHitResult;
		SavedHitResultFilterMode = GizmoFilterMode;
	}

	return BestHitResult;
}

bool UViewportInteractor::GetTransformAndForwardVector( FTransform& OutHandTransform, FVector& OutForwardVector ) const
{
	OutHandTransform = InteractorData.Transform;
	OutForwardVector = OutHandTransform.GetRotation().Vector();

	return true;
}

FVector UViewportInteractor::GetHoverLocation()
{
	FVector Result = FVector::ZeroVector;
	if (InteractorData.HoverLocation.IsSet())
	{
		Result = InteractorData.HoverLocation.GetValue();
	}
	else
	{
		FVector LaserStart, LaserEnd;
		GetLaserPointer(LaserStart, LaserEnd, true);
		Result = LaserEnd;
	}

	return Result;
}

bool UViewportInteractor::IsHovering() const
{
	return InteractorData.HoverLocation.IsSet();
}

bool UViewportInteractor::IsHoveringOverGizmo() const
{
	return InteractorData.HoveringOverTransformGizmoComponent.IsValid();
}

void UViewportInteractor::SetDraggingMode( const EViewportInteractionDraggingMode NewDraggingMode )
{
	InteractorData.DraggingMode = NewDraggingMode;
}

FViewportActionKeyInput* UViewportInteractor::GetActionWithName( const FName InActionName )
{
	FViewportActionKeyInput* ResultedAction = nullptr;
	for ( auto It = KeyToActionMap.CreateIterator(); It; ++It )
	{
		if ( It.Value().ActionType == InActionName )
		{
			ResultedAction = &(It->Value);
			break;
		}
	}
	return ResultedAction;
}

float UViewportInteractor::GetDragHapticFeedbackStrength() const
{
	return VI::DragHapticFeedbackStrength->GetFloat();
}

bool UViewportInteractor::IsHoveringOverPriorityType() const
{
	return InteractorData.bHitResultIsPriorityType;
}

bool UViewportInteractor::IsHoveringOverSelectedActor() const
{
	bool bResult = false;

	if (InteractorData.LastHoveredActorComponent != nullptr && InteractorData.HoverLocation.IsSet())
	{
		USelection* SelectedActors = GEditor->GetSelectedActors();
		for (FSelectionIterator SelectionIt(*SelectedActors); SelectionIt; ++SelectionIt)
		{
			AActor* Actor = CastChecked<AActor>(*SelectionIt);
			if (InteractorData.LastHoveredActorComponent->GetOwner() == Actor)
			{
				bResult = true;
				break;
			}
		}
	}

	return bResult;
}


void UViewportInteractor::ResetLaserEnd()
{
	SavedLaserPointerEnd.Reset();
}


void UViewportInteractor::SetHitResultGizmoFilterMode(EHitResultGizmoFilterMode newFilter)
{
	CurrentHitResultGizmoFilterMode = newFilter;
}


EHitResultGizmoFilterMode UViewportInteractor::GetHitResultGizmoFilterMode() const
{
	return CurrentHitResultGizmoFilterMode;
}

void UViewportInteractor::SetCanCarry(const bool bInCanCarry)
{
	bCanCarry = bInCanCarry;
}

bool UViewportInteractor::CanCarry() const
{
	return bCanCarry;
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
