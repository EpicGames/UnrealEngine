// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportTransformer.h"
#include "ViewportWorldInteraction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportTransformer)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


void UViewportTransformer::Init( UViewportWorldInteraction* InitViewportWorldInteraction )
{
	this->ViewportWorldInteraction = InitViewportWorldInteraction;
}


void UViewportTransformer::Shutdown()
{
	this->ViewportWorldInteraction = nullptr;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
