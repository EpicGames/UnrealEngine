// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportWorldInteraction.h"

#include "Materials/MaterialInterface.h"
#include "Misc/AxisDisplayInfo.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "EngineGlobals.h"
#include "Engine/OverlapResult.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/Selection.h"
#include "DrawDebugHelpers.h"
#include "EditorModeManager.h"
#include "UnrealEdGlobals.h"
#include "MouseCursorInteractor.h"
#include "ViewportInteractableInterface.h"
#include "ViewportDragOperation.h"
#include "VIGizmoHandle.h"
#include "Gizmo/VIPivotTransformGizmo.h"
#include "IViewportInteractionModule.h"
#include "ViewportTransformer.h"
#include "ActorTransformer.h"
#include "SnappingUtils.h"
#include "ScopedTransaction.h"
#include "DrawDebugHelpers.h"
#include "LevelEditorActions.h"
#include "Framework/Application/SlateApplication.h"

//Sound
#include "Kismet/GameplayStatics.h"
#include "ViewportInteractionStyle.h"
#include "ViewportInteractionAssetContainer.h"

// For actor placement
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "EngineUtils.h"
#include "ActorViewportTransformable.h"

// Preprocessor input
#include "ViewportInteractionInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "VISettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportWorldInteraction)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "ViewportWorldInteraction"

namespace VI
{
	static FAutoConsoleVariable GizmoScaleInDesktop( TEXT( "VI.GizmoScaleInDesktop" ), 0.35f, TEXT( "How big the transform gizmo should be when used in desktop mode" ) );
	static FAutoConsoleVariable AllowVerticalWorldMovement( TEXT( "VI.AllowVerticalWorldMovement" ), 1, TEXT( "Whether you can move your tracking space away from the origin or not" ) );
	static FAutoConsoleVariable AllowWorldRotationPitchAndRoll( TEXT( "VI.AllowWorldRotationPitchAndRoll" ), 0, TEXT( "When enabled, you'll not only be able to yaw, but also pitch and roll the world when rotating by gripping with two hands" ) );
	static FAutoConsoleVariable WorldScalingDragThreshold( TEXT( "VI.WorldScalingDragThreshold" ), 7.0f, TEXT( "How much you need to perform a scale gesture before world scaling starts to happen." ) );
	static FAutoConsoleVariable WorldRotationDragThreshold( TEXT( "VI.WorldRotationDragThreshold" ), 8.0f, TEXT( "How much (degrees) you need to perform a rotation gesture before world rotation starts to happen." ) );
	static FAutoConsoleVariable InertiaVelocityBoost( TEXT( "VI.InertiaVelocityBoost" ), 0.5f, TEXT( "How much to scale object velocity when releasing dragged simulating objects in Simulate mode" ) );
	static FAutoConsoleVariable SweepPhysicsWhileSimulating( TEXT( "VI.SweepPhysicsWhileSimulating" ), 0, TEXT( "If enabled, simulated objects won't be able to penetrate other objects while being dragged in Simulate mode" ) );
	static FAutoConsoleVariable PlacementInterpolationDuration( TEXT( "VI.PlacementInterpolationDuration" ), 0.6f, TEXT( "How long we should interpolate newly-placed objects to their target location." ) );
	static FAutoConsoleVariable PlacementOffsetScaleWhileSimulating( TEXT( "VI.PlacementOffsetScaleWhileSimulating" ), 0.25f, TEXT( "How far to additionally offset objects (as a scalar percentage of the gizmo bounds) from the placement impact point while simulate mode is active" ) );

	static FAutoConsoleVariable SmoothSnap( TEXT( "VI.SmoothSnap" ), 1, TEXT( "When enabled with grid snap, transformed objects will smoothly blend to their new location (instead of teleporting instantly)" ) );
	static FAutoConsoleVariable SmoothSnapSpeed( TEXT( "VI.SmoothSnapSpeed" ), 30.0f, TEXT( "How quickly objects should interpolate to their new position when grid snapping is enabled" ) );
	static FAutoConsoleVariable ElasticSnap( TEXT( "VI.ElasticSnap" ), 1, TEXT( "When enabled with grid snap, you can 'pull' objects slightly away from their snapped position" ) );
	static FAutoConsoleVariable ElasticSnapStrength( TEXT( "VI.ElasticSnapStrength" ), 0.3f, TEXT( "How much objects should 'reach' toward their unsnapped position when elastic snapping is enabled with grid snap" ) );
	static FAutoConsoleVariable ScaleMax( TEXT( "VI.ScaleMax" ), 6000.0f, TEXT( "Maximum world scale in centimeters" ) );
	static FAutoConsoleVariable ScaleMin( TEXT( "VI.ScaleMin" ), 10.0f, TEXT( "Minimum world scale in centimeters" ) );
	static FAutoConsoleVariable DragAtLaserImpactInterpolationDuration( TEXT( "VI.DragAtLaserImpactInterpolationDuration" ), 0.1f, TEXT( "How long we should interpolate objects between positions when dragging under the laser's impact point" ) );
	static FAutoConsoleVariable DragAtLaserImpactInterpolationThreshold( TEXT( "VI.DragAtLaserImpactInterpolationThreshold" ), 5.0f, TEXT( "Minimum distance jumped between frames before we'll force interpolation mode to activated" ) );

	static FAutoConsoleVariable ForceGizmoPivotToCenterOfObjectsBounds( TEXT( "VI.ForceGizmoPivotToCenterOfObjectsBounds" ), 0, TEXT( "When enabled, the gizmo's pivot will always be centered on the selected objects.  Otherwise, we use the pivot of the last selected object." ) );
	static FAutoConsoleVariable GizmoHandleHoverScale( TEXT( "VI.GizmoHandleHoverScale" ), 1.5f, TEXT( "How much to scale up transform gizmo handles when hovered over" ) );
	static FAutoConsoleVariable GizmoHandleHoverAnimationDuration( TEXT( "VI.GizmoHandleHoverAnimationDuration" ), 0.1f, TEXT( "How quickly to animate gizmo handle hover state" ) );
	static FAutoConsoleVariable ShowTransformGizmo( TEXT( "VI.ShowTransformGizmo" ), 1, TEXT( "Whether the transform gizmo should be shown for selected objects" ) );
	static FAutoConsoleVariable DragTranslationVelocityStopEpsilon( TEXT( "VI.DragTranslationVelocityStopEpsilon" ), KINDA_SMALL_NUMBER, TEXT( "When dragging inertia falls below this value (cm/frame), we'll stop inertia and finalize the drag" ) );
	static FAutoConsoleVariable SnapGridSize( TEXT( "VI.SnapGridSize" ), 3.0f, TEXT( "How big the snap grid should be.  At 1.0, this will be the maximum of the gizmo's bounding box and a multiple of the current grid snap size" ) );
	static FAutoConsoleVariable SnapGridLineWidth( TEXT( "VI.SnapGridLineWidth" ), 3.0f, TEXT( "Width of the grid lines on the snap grid" ) );
	static FAutoConsoleVariable MinVelocityForInertia( TEXT( "VI.MinVelocityForInertia" ), 1.0f, TEXT( "Minimum velocity (in cm/frame in unscaled room space) before inertia will kick in when releasing objects (or the world)" ) );
	static FAutoConsoleVariable GridHapticFeedbackStrength( TEXT( "VI.GridHapticFeedbackStrength" ), 0.4f, TEXT( "Default strength for haptic feedback when moving across grid points" ) );
	static FAutoConsoleVariable ActorSnap(TEXT("VI.ActorSnap"), 0, TEXT("Whether or not to snap to Actors in the scene. Off by default, set to 1 to enable."));
	static FAutoConsoleVariable AlignCandidateDistance(TEXT("VI.AlignCandidateDistance"), 2.0f, TEXT("The distance candidate actors can be from our transformable (in multiples of our transformable's size"));
	static FAutoConsoleVariable ForceSnapDistance(TEXT("VI.ForceSnapDistance"), 25.0f, TEXT("The distance (in % of transformable size) where guide lines indicate that actors are aligned"));
	static FAutoConsoleVariable AllowCarryingCertainObjects(TEXT("VI.AllowCarryingCertainObjects"), 1, TEXT("When enabled, allows the user to freely move and rotate certain selected objects with a one-hand drag."));
	static FAutoConsoleVariable CarrySmoothingLerpAlpha(TEXT("VI.CarrySmoothingLerpAlpha"), 1.0f, TEXT("How much to smooth out movement of the object you're carrying."));

	static FAutoConsoleVariable ForceShowCursor(TEXT("VI.ForceShowCursor"), 0, TEXT("Whether or not the mirror window's cursor should be enabled. Off by default, set to 1 to enable."));
	static FAutoConsoleVariable SFXMultiplier(TEXT("VI.SFXMultiplier"), 1.5f, TEXT("Default Sound Effect Volume Multiplier"));

	static FAutoConsoleVariable NavigationMode(TEXT("VI.NavigationMode"), 0, TEXT("VR NavigationMode"));
	static FAutoConsoleVariable MaxFlightSpeed(TEXT("VI.MaxFlightSpeed"), 11.0f, TEXT("Maximum Superman speed"));
	static FAutoConsoleVariable DragScale(TEXT("VI.DragScale"), 1.0f, TEXT("Scales the translation when dragging yourself through the world"));
	static FAutoConsoleVariable LowSpeedInertiaDamping(TEXT("VI.LowSpeedInertiaDamping"), 0.94f, TEXT("Low Speed Inertia Damping multiplier"));
	static FAutoConsoleVariable HighSpeedInertiaDamping(TEXT("VI.HighSpeedInertiaDamping"), 0.99f, TEXT("Hight Speed Inertia Damping multiplier"));

}

const TCHAR* UViewportWorldInteraction::AssetContainerPath = TEXT("/Engine/VREditor/ViewportInteractionAssetContainerData");

struct FGuideData
{
	const AActor* AlignedActor;
	FVector LocalOffset;
	FVector SnapPoint;
	FVector GuideStart;
	FVector GuideEnd;
	FColor GuideColor;
	float DrawAlpha;
	double GuideLength;
};

// @todo vreditor: Hacky inline implementation of a double vector.  Move elsewhere and polish or get rid.
struct DVector
{
	double X;
	double Y;
	double Z;

	DVector()
	{
	}

	DVector( const FVector& V )
		: X( V.X ), Y( V.Y ), Z( V.Z )
	{
	}

	DVector( double InX, double InY, double InZ )
		: X( InX ), Y( InY ), Z( InZ )
	{
	}

	DVector operator+( const DVector& V ) const
	{
		return DVector( X + V.X, Y + V.Y, Z + V.Z );
	}

	DVector operator-( const DVector& V ) const
	{
		return DVector( X - V.X, Y - V.Y, Z - V.Z );
	}

	DVector operator*( double Scale ) const
	{
		return DVector( X * Scale, Y * Scale, Z * Scale );
	}

	double operator|( const DVector& V ) const
	{
		return X*V.X + Y*V.Y + Z*V.Z;
	}

	FVector ToFVector() const
	{
		return FVector( ( float ) X, ( float ) Y, ( float ) Z );
	}
};

// @todo vreditor: Move elsewhere or get rid
static void SegmentDistToSegmentDouble( DVector A1, DVector B1, DVector A2, DVector B2, DVector& OutP1, DVector& OutP2 )
{
	const double Epsilon = 1.e-10;

	// Segments
	const	DVector	S1 = B1 - A1;
	const	DVector	S2 = B2 - A2;
	const	DVector	S3 = A1 - A2;

	const	double	Dot11 = S1 | S1;	// always >= 0
	const	double	Dot22 = S2 | S2;	// always >= 0
	const	double	Dot12 = S1 | S2;
	const	double	Dot13 = S1 | S3;
	const	double	Dot23 = S2 | S3;

	// Numerator
	double	N1, N2;

	// Denominator
	const	double	D = Dot11*Dot22 - Dot12*Dot12;	// always >= 0
	double	D1 = D;		// T1 = N1 / D1, default D1 = D >= 0
	double	D2 = D;		// T2 = N2 / D2, default D2 = D >= 0

						// compute the line parameters of the two closest points
	if ( D < Epsilon )
	{
		// the lines are almost parallel
		N1 = 0.0;	// force using point A on segment S1
		D1 = 1.0;	// to prevent possible division by 0 later
		N2 = Dot23;
		D2 = Dot22;
	}
	else
	{
		// get the closest points on the infinite lines
		N1 = ( Dot12*Dot23 - Dot22*Dot13 );
		N2 = ( Dot11*Dot23 - Dot12*Dot13 );

		if ( N1 < 0.0 )
		{
			// t1 < 0.0 => the s==0 edge is visible
			N1 = 0.0;
			N2 = Dot23;
			D2 = Dot22;
		}
		else if ( N1 > D1 )
		{
			// t1 > 1 => the t1==1 edge is visible
			N1 = D1;
			N2 = Dot23 + Dot12;
			D2 = Dot22;
		}
	}

	if ( N2 < 0.0 )
	{
		// t2 < 0 => the t2==0 edge is visible
		N2 = 0.0;

		// recompute t1 for this edge
		if ( -Dot13 < 0.0 )
		{
			N1 = 0.0;
		}
		else if ( -Dot13 > Dot11 )
		{
			N1 = D1;
		}
		else
		{
			N1 = -Dot13;
			D1 = Dot11;
		}
	}
	else if ( N2 > D2 )
	{
		// t2 > 1 => the t2=1 edge is visible
		N2 = D2;

		// recompute t1 for this edge
		if ( ( -Dot13 + Dot12 ) < 0.0 )
		{
			N1 = 0.0;
		}
		else if ( ( -Dot13 + Dot12 ) > Dot11 )
		{
			N1 = D1;
		}
		else
		{
			N1 = ( -Dot13 + Dot12 );
			D1 = Dot11;
		}
	}

	// finally do the division to get the points' location
	const double T1 = ( FMath::Abs( N1 ) < Epsilon ? 0.0 : N1 / D1 );
	const double T2 = ( FMath::Abs( N2 ) < Epsilon ? 0.0 : N2 / D2 );

	// return the closest points
	OutP1 = A1 + S1 * T1;
	OutP2 = A2 + S2 * T2;
}

UViewportWorldInteraction::UViewportWorldInteraction():
	Super(),
	bDraggedSinceLastSelection( false ),
	LastDragGizmoStartTransform( FTransform::Identity ),
	TrackingTransaction( FTrackingTransaction() ),
	AppTimeEntered( FTimespan::Zero() ),
	DefaultOptionalViewportClient( nullptr ),
	LastFrameNumberInputWasPolled( 0 ),
	MotionControllerID( 0 ),	// @todo ViewportInteraction: We only support a single controller, and we assume the first controller are the motion controls
	LastWorldToMetersScale( 100.0f ),
	bSkipInteractiveWorldMovementThisFrame( false ),
	RoomTransformToSetOnFrame(),
	LowSpeedInertiaDamping( VI::LowSpeedInertiaDamping->GetFloat() ),
	HighSpeedInertiaDamping( VI::HighSpeedInertiaDamping->GetFloat() ),
	bAreTransformablesMoving( false ),
	bIsInterpolatingTransformablesFromSnapshotTransform( false ),
	bFreezePlacementWhileInterpolatingTransformables( false ),
	TransformablesInterpolationStartTime( FTimespan::Zero() ),
	TransformablesInterpolationDuration( 1.0f ),
	TransformGizmoActor( nullptr ),
	TransformGizmoClass( APivotTransformGizmo::StaticClass() ),
	GizmoLocalBounds( FBox(ForceInit) ),
	bShouldTransformGizmoBeVisible( true ),
	TransformGizmoScale( VI::GizmoScaleInDesktop->GetFloat() ),
	GizmoType(),
	SnapGridActor( nullptr ),
	SnapGridMeshComponent( nullptr ),
	SnapGridMID( nullptr ),
	DraggedInteractable( nullptr ),
	DefaultMouseCursorInteractor( nullptr ),
	DefaultMouseCursorInteractorRefCount( 0 ),
	bIsInVR( false ),
	bUseInputPreprocessor( false ),
	bAllowWorldMovement( true ),
	CurrentDeltaTime(0.0f),
	bShouldSuppressCursor(false),
	CurrentTickNumber(0),
	AssetContainer(nullptr),
	bPlayNextRefreshTransformGizmoSound(true),
	InputProcessor()
{
}


void UViewportWorldInteraction::Init()
{
	InitColors();

	AppTimeEntered = FTimespan::FromSeconds( FApp::GetCurrentTime() );

	// Setup the asset container.
	AssetContainer = LoadAssetContainer();

	// Start with the default transformer
	SetTransformer( nullptr );

	//Spawn the transform gizmo at init so we do not hitch when selecting our first object
	SpawnTransformGizmoIfNeeded();

	const bool bShouldBeVisible = false;
	const bool bPropagateToChildren = true;
	TransformGizmoActor->GetRootComponent()->SetVisibility( bShouldBeVisible, bPropagateToChildren );

	/** This will make sure this is not ticking after the editor has been closed. */
	GEditor->OnEditorClose().AddUObject( this, &UViewportWorldInteraction::Shutdown );

	// We need a mouse cursor!
	this->AddMouseCursorInteractor();

	SetActive(true);

	CandidateActors.Reset();

	// Create and add the input pre-processor to the slate application.
	InputProcessor = MakeShareable(new FViewportInteractionInputProcessor(this));
	FSlateApplication::Get().RegisterInputPreProcessor(InputProcessor);

	// Pretend that actor selection changed, so that our gizmo refreshes right away based on which objects are selected
	GEditor->NoteSelectionChange();
	GEditor->SelectNone(true, true, false);

	CurrentTickNumber = 0;
}

void UViewportWorldInteraction::InitColors()
{
	Colors.SetNumZeroed((int32)EColors::TotalCount);
	{
		Colors[(int32)EColors::DefaultColor] = FLinearColor(0.7f, 0.7f, 0.7f, 1.0f);
		Colors[(int32)EColors::Forward] = AxisDisplayInfo::GetAxisColor(EAxisList::X);
		Colors[(int32)EColors::Right] = AxisDisplayInfo::GetAxisColor(EAxisList::Y);
		Colors[(int32)EColors::Up] = AxisDisplayInfo::GetAxisColor(EAxisList::Z);
		Colors[(int32)EColors::GizmoHover] = FLinearColor::Yellow;
		Colors[(int32)EColors::GizmoDragging] = FLinearColor::Yellow;
	}
}

void UViewportWorldInteraction::Shutdown()
{
	SetActive(false);

	if( DefaultMouseCursorInteractorRefCount == 1 )
	{
		this->ReleaseMouseCursorInteractor();
	}

	DestroyActors();
	if (DefaultOptionalViewportClient != nullptr)
	{
		DefaultOptionalViewportClient->ShowWidget(true);
		DefaultOptionalViewportClient = nullptr;
	}

	AppTimeEntered = FTimespan::Zero();

	for ( UViewportInteractor* Interactor : Interactors )
	{
		Interactor->Shutdown();
		Interactor->MarkAsGarbage();
	}

	Interactors.Empty();
	Transformables.Empty();
	Colors.Empty();

	OnHoverUpdateEvent.Clear();
	OnPreviewInputActionEvent.Clear();
	OnInputActionEvent.Clear();
	OnKeyInputEvent.Clear();
	OnAxisInputEvent.Clear();
	OnStopDraggingEvent.Clear();

	TransformGizmoActor = nullptr;
	SnapGridActor = nullptr;
	SnapGridMeshComponent = nullptr;
	SnapGridMID = nullptr;
	DraggedInteractable = nullptr;
	if( ViewportTransformer != nullptr )
	{
		ViewportTransformer->Shutdown();
		ViewportTransformer = nullptr;
	}

	AssetContainer = nullptr;

	GizmoType.Reset();

	// Remove the input pre-processor
	if (InputProcessor.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(InputProcessor);
		InputProcessor.Reset();
	}
	USelection::SelectionChangedEvent.RemoveAll( this );
	GEditor->OnEditorClose().RemoveAll( this );
}

void UViewportWorldInteraction::TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState)
{
	check(NewWorld != nullptr);

	Super::TransitionWorld(NewWorld, TransitionState);

	if (TransitionState == EEditorWorldExtensionTransitionState::TransitionAll ||
		TransitionState == EEditorWorldExtensionTransitionState::TransitionNonPIEOnly)
	{
		for (UViewportInteractor* Interactor : Interactors)
		{
			Interactor->Rename(nullptr, NewWorld->PersistentLevel);
		}
	}
}

void UViewportWorldInteraction::EnteredSimulateInEditor()
{
	// Make sure transformables get updated
	GEditor->NoteSelectionChange();
}

void UViewportWorldInteraction::LeftSimulateInEditor(UWorld* SimulateWorld)
{
	// Make sure transformables get updated
	GEditor->NoteSelectionChange();
}

void UViewportWorldInteraction::OnEditorClosed()
{
	if( IsActive() )
	{
		Shutdown();
	}
}

void UViewportWorldInteraction::Tick( float DeltaSeconds )
{
	CurrentDeltaTime = DeltaSeconds;

	if( RoomTransformToSetOnFrame.IsSet() )
	{
		const uint32 SetOnFrameNumber = RoomTransformToSetOnFrame.GetValue().Get<1>();
		if( SetOnFrameNumber == CurrentTickNumber )
		{
			const FTransform& NewRoomTransform = RoomTransformToSetOnFrame.GetValue().Get<0>();
			SetRoomTransform( NewRoomTransform );
			RoomTransformToSetOnFrame.Reset();
		}
	}

	OnPreWorldInteractionTickEvent.Broadcast( DeltaSeconds );

	// Get the latest interactor data, and fill in our its data with fresh transforms
	PollInputIfNeeded();

	// Update hover. Note that hover can also be updated when ticking our sub-systems below.
	HoverTick( DeltaSeconds );

	InteractionTick( DeltaSeconds );

	// Update all the interactors
	for ( UViewportInteractor* Interactor : Interactors )
	{
		Interactor->Tick( DeltaSeconds );
	}

	OnPostWorldInteractionTickEvent.Broadcast( DeltaSeconds );

	for (UViewportInteractor* Interactor : Interactors)
	{
		Interactor->ResetLaserEnd();
	}

	CurrentTickNumber++;
}

void UViewportWorldInteraction::AddInteractor( UViewportInteractor* Interactor )
{
	Interactor->SetWorldInteraction( this );
	Interactors.AddUnique( Interactor ); 
}

void UViewportWorldInteraction::RemoveInteractor( UViewportInteractor* Interactor )
{
	Interactor->Shutdown();
	Interactor->RemoveOtherInteractor();
	Interactors.Remove( Interactor );
}


void UViewportWorldInteraction::SetTransformer( UViewportTransformer* NewTransformer )
{
	// Clear all existing transformables
	const bool bNewObjectsSelected = false;
	SetTransformables( TArray< TUniquePtr< class FViewportTransformable > >(), bNewObjectsSelected );

	if( ViewportTransformer != nullptr )
	{
		ViewportTransformer->Shutdown();
		ViewportTransformer = nullptr;
	}

	ViewportTransformer = NewTransformer;

	if( ViewportTransformer == nullptr )
	{
		ViewportTransformer = NewObject<UActorTransformer>();
	}
	check( ViewportTransformer != nullptr );

	ViewportTransformer->Init( this );
}


void UViewportWorldInteraction::SetTransformables( TArray< TUniquePtr< class FViewportTransformable > >&& NewTransformables, const bool bNewObjectsSelected )
{
	// We're dealing with a new set of transformables, so clear out anything that was going on with the old ones.
	bDraggedSinceLastSelection = false;
	LastDragGizmoStartTransform = FTransform::Identity;
	bIsInterpolatingTransformablesFromSnapshotTransform = false;
	bFreezePlacementWhileInterpolatingTransformables = false;
	TransformablesInterpolationStartTime = FTimespan::Zero();
	TransformablesInterpolationDuration = 1.0f;

	// Clear our last dragging mode on all interactors, so no inertia from the last objects we dragged will
	// be applied to the new transformables.
	for( UViewportInteractor* Interactor : Interactors )
	{
		FViewportInteractorData& InteractorData = Interactor->GetInteractorData();
		InteractorData.LastDraggingMode = EViewportInteractionDraggingMode::Nothing;
	}

	Transformables = MoveTemp( NewTransformables );

	RefreshTransformGizmo( bNewObjectsSelected );
}

void UViewportWorldInteraction::SetDefaultOptionalViewportClient(const TSharedPtr<class FEditorViewportClient>& InEditorViewportClient)
{
	if (InEditorViewportClient.IsValid())
	{
		DefaultOptionalViewportClient = InEditorViewportClient.Get();
		DefaultOptionalViewportClient->ShowWidget(false);
	}
}

void UViewportWorldInteraction::PairInteractors( UViewportInteractor* FirstInteractor, UViewportInteractor* SecondInteractor )
{
	FirstInteractor->SetOtherInteractor( SecondInteractor );
	SecondInteractor->SetOtherInteractor( FirstInteractor );
}

bool UViewportWorldInteraction::InputKey( FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event )
{
	bool bWasHandled = false;

	if( IsActive() && !bWasHandled )
	{
		check( InViewportClient != nullptr );

		// Don't bother processing clicks if Alt is held down, as that's used for orbit in level viewports
		FInputEventState InputState( InViewportClient->Viewport, Key, Event );
		const bool bAltOrbitDragging = InputState.IsAltButtonPressed() && !InputState.IsCtrlButtonPressed() && !InputState.IsShiftButtonPressed() && Key == EKeys::LeftMouseButton;
		if( !bAltOrbitDragging )
		{
			OnKeyInputEvent.Broadcast(InViewportClient, Key, Event, bWasHandled);

			if( !bWasHandled || !IsActive() )
			{
				for( UViewportInteractor* Interactor : Interactors )
				{
					bWasHandled = Interactor->HandleInputKey( *InViewportClient, Key, Event );

					// Stop iterating if the input was handled by an Interactor
					if( bWasHandled || !IsActive() )
					{
						break;
					}
				}
			}
		}
	}

	if( bWasHandled )
	{
		// This is a temporary workaround so that we don't steal the handling of the drag tool from the viewport client
		FInputEventState InputState( InViewportClient->Viewport, Key, Event );
		const bool bMarqueeSelecting = InputState.IsAltButtonPressed() && InputState.IsCtrlButtonPressed() && Key == EKeys::LeftMouseButton;
		if( bMarqueeSelecting )
		{
			bWasHandled = false;
		}
	}

	return bWasHandled;
}

bool UViewportWorldInteraction::PreprocessedInputKey( const FKey Key, const EInputEvent Event )
{
	if( DefaultOptionalViewportClient != nullptr && bUseInputPreprocessor == true )
	{
		return InputKey( DefaultOptionalViewportClient, DefaultOptionalViewportClient->Viewport, Key, Event );
	}

	return false;
}

bool UViewportWorldInteraction::InputAxis( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime )
{
	bool bWasHandled = false;

	if( IsActive() )
	{
		check( InViewportClient != nullptr );
		OnAxisInputEvent.Broadcast( InViewportClient, ControllerId, Key, Delta, DeltaTime, bWasHandled );

		if( !bWasHandled || !IsActive() )
		{
			for( UViewportInteractor* Interactor : Interactors )
			{
				bWasHandled = Interactor->HandleInputAxis( *InViewportClient, Key, Delta, DeltaTime );

				// Stop iterating if the input was handled by an interactor
				if( bWasHandled || !IsActive() )
				{
					break;
				}
			}
		}
	}

	return bWasHandled;
}

bool UViewportWorldInteraction::PreprocessedInputAxis( const int32 ControllerId, const FKey Key, const float Delta, const double DeltaTime )
{
	if( DefaultOptionalViewportClient != nullptr && bUseInputPreprocessor == true )
	{
		return InputAxis( DefaultOptionalViewportClient, DefaultOptionalViewportClient->Viewport, ControllerId, Key, Delta, (float)DeltaTime );
	}

	return false;
}

FTransform UViewportWorldInteraction::GetRoomTransform() const
{
	FTransform RoomTransform;
	if( DefaultOptionalViewportClient != nullptr )
	{
		RoomTransform = FTransform(
			DefaultOptionalViewportClient->GetViewRotation().Quaternion(),
			DefaultOptionalViewportClient->GetViewLocation(),
			FVector( 1.0f ) );
	}
	return RoomTransform;
}

FTransform UViewportWorldInteraction::GetRoomSpaceHeadTransform() const
{
	FTransform HeadTransform = FTransform::Identity;
	if( HaveHeadTransform() )
	{
		FQuat RoomSpaceHeadOrientation;
		FVector RoomSpaceHeadLocation;
		GEngine->XRSystem->GetCurrentPose( IXRTrackingSystem::HMDDeviceId, /* Out */ RoomSpaceHeadOrientation, /* Out */ RoomSpaceHeadLocation );

		HeadTransform = FTransform(
			RoomSpaceHeadOrientation,
			RoomSpaceHeadLocation,
			FVector( 1.0f ) );
	}

	return HeadTransform;
}

FTransform UViewportWorldInteraction::GetHeadTransform() const
{
	return GetRoomSpaceHeadTransform() * GetRoomTransform();
}

void UViewportWorldInteraction::SetHeadTransform( const FTransform& NewHeadTransform )
{
	if (DefaultOptionalViewportClient != nullptr)
	{
		FRotator UseRotation( NewHeadTransform.GetRotation().Rotator() );

		// adjust the yaw by the difference of the flag and the HMD
		double YawDifference = GetHeadTransform().GetRotation().Rotator().Yaw - UseRotation.Yaw;
		UseRotation.Yaw = GetRoomTransform().GetRotation().Rotator().Yaw - YawDifference;
		UseRotation.Pitch = 0.0f;
		UseRotation.Roll = 0.0f;

		FVector Location( NewHeadTransform.GetLocation() );
		// get HMD location in room space but scaled to the floor.
		FVector HMDLocationOffset( GetRoomSpaceHeadTransform().GetLocation() * FVector( 1.0f, 1.0f, 1.0f ) );

		// rotate the offset vector by the rotator
		HMDLocationOffset = UseRotation.RotateVector( HMDLocationOffset );
		// offset the room location by the HMD
		Location -= HMDLocationOffset;
		
		FTransform NewRoomTransform;
		NewRoomTransform.SetLocation( Location );
		NewRoomTransform.SetRotation( UseRotation.Quaternion() );
		RoomTransformToSetOnFrame = MakeTuple( NewRoomTransform, CurrentTickNumber + 1 );
	}
}

bool UViewportWorldInteraction::HaveHeadTransform() const
{
	return DefaultOptionalViewportClient != nullptr && GEngine->XRSystem.IsValid() && GEngine->StereoRenderingDevice.IsValid() && GEngine->StereoRenderingDevice->IsStereoEnabled();
}


void UViewportWorldInteraction::SetRoomTransform( const FTransform& NewRoomTransform )
{
	if( DefaultOptionalViewportClient != nullptr )
	{
		DefaultOptionalViewportClient->SetViewLocation( NewRoomTransform.GetLocation() );
		DefaultOptionalViewportClient->SetViewRotation( NewRoomTransform.GetRotation().Rotator() );

		// Forcibly dirty the viewport camera location
		const bool bDollyCamera = false;
		DefaultOptionalViewportClient->MoveViewportCamera( FVector::ZeroVector, FRotator::ZeroRotator, bDollyCamera );
	}
}

void UViewportWorldInteraction::SetRoomTransformForNextFrame(const FTransform& NewRoomTransform)
{
	RoomTransformToSetOnFrame = MakeTuple(NewRoomTransform, CurrentTickNumber + 1);

}

float UViewportWorldInteraction::GetWorldScaleFactor() const
{
	if (GetWorld() && GetWorld()->GetWorldSettings())
	{
		return GetWorld()->GetWorldSettings()->WorldToMeters / 100.0f;
	}
	else
	{
		return 1.0f;
	}
}

FEditorViewportClient* UViewportWorldInteraction::GetDefaultOptionalViewportClient() const
{
	return DefaultOptionalViewportClient;
}

void UViewportWorldInteraction::Undo()
{
	PlaySound(AssetContainer->UndoSound, TransformGizmoActor->GetActorLocation());
	bPlayNextRefreshTransformGizmoSound = false;
	ExecCommand(FString("TRANSACTION UNDO"));
}

void UViewportWorldInteraction::Redo()
{
	PlaySound(AssetContainer->RedoSound, TransformGizmoActor->GetActorLocation());
	bPlayNextRefreshTransformGizmoSound = false;
	ExecCommand(FString("TRANSACTION REDO"));
}

void UViewportWorldInteraction::DeleteSelectedObjects()
{
	if (FLevelEditorActionCallbacks::Delete_CanExecute())
	{
		ExecCommand(FString("DELETE"));
	}
}

void UViewportWorldInteraction::Copy()
{
	if (FLevelEditorActionCallbacks::Copy_CanExecute())
	{
		ExecCommand(FString("EDIT COPY"));
	}
}

void UViewportWorldInteraction::Paste()
{
	if (FLevelEditorActionCallbacks::Paste_CanExecute())
	{
		// @todo vreditor: Needs "paste here" style pasting (TO=HERE), but with ray
		ExecCommand(FString("EDIT PASTE"));
	}
}

void UViewportWorldInteraction::Duplicate()
{
	if (FLevelEditorActionCallbacks::Duplicate_CanExecute())
	{
		ExecCommand(FString("DUPLICATE"));
	}
}

void UViewportWorldInteraction::Deselect()
{
	GEditor->SelectNone( true, true );
}

void UViewportWorldInteraction::HoverTick( const float DeltaTime )
{
	for ( UViewportInteractor* Interactor : Interactors )
	{
		bool bWasHandled = false;
		FViewportInteractorData& InteractorData = Interactor->GetInteractorData();
		
		Interactor->ResetHoverState();
		
		FHitResult HitHoverResult = Interactor->GetHitResultFromLaserPointer();

		const bool bIsHoveringOverTransformGizmo =
			HitHoverResult.HitObjectHandle.IsValid() &&
			HitHoverResult.HitObjectHandle == TransformGizmoActor;

		// Prefer transform gizmo hover over everything else
		if( !bIsHoveringOverTransformGizmo )
		{
			FVector HoverImpactPoint = FVector::ZeroVector;
			OnHoverUpdateEvent.Broadcast( Interactor, /* In/Out */ HoverImpactPoint, /* In/Out */ bWasHandled );
			if( bWasHandled )
			{
				Interactor->SetHoverLocation(HoverImpactPoint);
			}
		}
		
		if ( !bWasHandled )
		{
			UActorComponent* NewHoveredActorComponent = nullptr;

			if ( HitHoverResult.HitObjectHandle.IsValid() )
			{
				USceneComponent* HoveredActorComponent = HitHoverResult.GetComponent();	

				if ( HoveredActorComponent && IsInteractableComponent( HoveredActorComponent ) )
				{
					AActor* Actor = HitHoverResult.HitObjectHandle.FetchActor();
					HoveredObjects.Add( FViewportHoverTarget( Actor ) );

					Interactor->SetHoverLocation(HitHoverResult.ImpactPoint);

					bWasHandled = true;

					if ( Actor == TransformGizmoActor )
					{
						NewHoveredActorComponent = HoveredActorComponent;
						InteractorData.HoveringOverTransformGizmoComponent = HoveredActorComponent;
						InteractorData.HoverHapticCheckLastHoveredGizmoComponent = HitHoverResult.GetComponent();
					}
					else
					{
						// This is a "manual" interface Cast<> to avoid emitting a spurious deprecation warning.
						// This code will all be deleted in UE 5.9.
						if (Actor->GetClass()->ImplementsInterface(UViewportInteractableInterface::StaticClass()))
						{
							IViewportInteractableInterface* Interactable =
								(IViewportInteractableInterface*)Actor->GetInterfaceAddress(UViewportInteractableInterface::StaticClass());

							NewHoveredActorComponent = HoveredActorComponent;

							// Check if the current hovered component of the interactor is different from the hitresult component
							if ( NewHoveredActorComponent != InteractorData.LastHoveredActorComponent && !IsOtherInteractorHoveringOverComponent( Interactor, NewHoveredActorComponent ) )
							{
								Interactable->OnHoverEnter( Interactor, HitHoverResult );
							}

							Interactable->OnHover( Interactor );
						}
					}
				}
			}

			// Leave hovered interactable
			//@todo ViewportInteraction: This does not take into account when the other interactors are already hovering over this interactable
			if ( InteractorData.LastHoveredActorComponent != nullptr && ( InteractorData.LastHoveredActorComponent != NewHoveredActorComponent || NewHoveredActorComponent == nullptr ) 
				&& !IsOtherInteractorHoveringOverComponent( Interactor, NewHoveredActorComponent ) )
			{
				AActor* HoveredActor = InteractorData.LastHoveredActorComponent->GetOwner();
				if ( HoveredActor )
				{
					// This is a "manual" interface Cast<> to avoid emitting a spurious deprecation warning.
					// This code will all be deleted in UE 5.9.
					if (HoveredActor->GetClass()->ImplementsInterface(UViewportInteractableInterface::StaticClass()))
					{
						IViewportInteractableInterface* Interactable =
							(IViewportInteractableInterface*)HoveredActor->GetInterfaceAddress(UViewportInteractableInterface::StaticClass());

						Interactable->OnHoverLeave( Interactor, NewHoveredActorComponent );
					}
				}
			}

			//Update the hovered actor component with the new component
			InteractorData.LastHoveredActorComponent = NewHoveredActorComponent;
		}

		if (InteractorData.LastHoveredActorComponent == nullptr && HitHoverResult.GetComponent() != nullptr)
		{
			InteractorData.LastHoveredActorComponent = HitHoverResult.GetComponent();
		}
	}
}

void UViewportWorldInteraction::InteractionTick( const float DeltaTime )
{
	const FTimespan CurrentTime = FTimespan::FromSeconds( FPlatformTime::Seconds() );
	const float WorldToMetersScale = GetWorld()->GetWorldSettings()->WorldToMeters;

	// Update viewport with any objects that are currently hovered over
	{
		if( DefaultOptionalViewportClient != nullptr )
		{
			const bool bUseEditorSelectionHoverFeedback = GEditor != NULL && GetDefault<ULevelEditorViewportSettings>()->bEnableViewportHoverFeedback;
			if( bUseEditorSelectionHoverFeedback && DefaultOptionalViewportClient->IsLevelEditorClient())
			{
				FLevelEditorViewportClient* LevelEditorViewportClient = static_cast<FLevelEditorViewportClient*>( DefaultOptionalViewportClient );
				LevelEditorViewportClient->UpdateHoveredObjects( HoveredObjects );
			}
		}
		// This will be filled in again during the next input update
		HoveredObjects.Reset();
	}

	const float WorldScaleFactor = GetWorldScaleFactor();

	// Move selected actors
	UViewportInteractor* DraggingWithInteractor = nullptr;
	UViewportInteractor* AssistingDragWithInteractor = nullptr;

	for ( UViewportInteractor* Interactor : Interactors )
	{
		bool bCanSlideRayLength = false;
		FViewportInteractorData& InteractorData = Interactor->GetInteractorData();

		if( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
			InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
			InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
		{
			check( DraggingWithInteractor == nullptr );	// Only support dragging one thing at a time right now!
			DraggingWithInteractor = Interactor;

			if (!InteractorData.bDraggingWithGrabberSphere)
			{
				bCanSlideRayLength = true;
			}
		}

		if ( InteractorData.DraggingMode == EViewportInteractionDraggingMode::AssistingDrag )
		{
			check( AssistingDragWithInteractor == nullptr );	// Only support assisting one thing at a time right now!
			AssistingDragWithInteractor = Interactor;

			if( !InteractorData.bDraggingWithGrabberSphere )
			{
				bCanSlideRayLength = true;
			}
		}

		if ( bCanSlideRayLength )
		{
			Interactor->CalculateDragRay( InteractorData.DragRayLength, InteractorData.DragRayLengthVelocity );
		}
	}

	UViewportInteractor* InertiaFromInteractor = nullptr;
	for ( UViewportInteractor* Interactor : Interactors )
	{
		FViewportInteractorData& InteractorData = Interactor->GetInteractorData();
			
		if( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
			InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
			InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact ||
			( InteractorData.DraggingMode == EViewportInteractionDraggingMode::World && bAllowWorldMovement ) )
		{
			// Are we dragging with two interactors ?
			UViewportInteractor* OtherInteractor = nullptr;
			if ( AssistingDragWithInteractor != nullptr )
			{
				OtherInteractor = AssistingDragWithInteractor;
			}

			UViewportInteractor* OtherInteractorThatWasAssistingDrag = GetOtherInteractorIntertiaContribute( Interactor );

			FVector DraggedTo = InteractorData.Transform.GetLocation();
			FVector DragDelta = DraggedTo - InteractorData.LastTransform.GetLocation();
			FVector DragDeltaFromStart = DraggedTo - InteractorData.ImpactLocationAtDragStart;

			FVector OtherHandDraggedTo = FVector::ZeroVector;
			FVector OtherHandDragDelta = FVector::ZeroVector;
			FVector OtherHandDragDeltaFromStart = FVector::ZeroVector;
			if( OtherInteractor != nullptr )
			{
				OtherHandDraggedTo = OtherInteractor->GetInteractorData().Transform.GetLocation();
				OtherHandDragDelta = OtherHandDraggedTo - OtherInteractor->GetInteractorData().LastTransform.GetLocation();
				OtherHandDragDeltaFromStart = DraggedTo - OtherInteractor->GetInteractorData().ImpactLocationAtDragStart;
			}
			else if( OtherInteractorThatWasAssistingDrag != nullptr )
			{
				OtherHandDragDelta = OtherInteractorThatWasAssistingDrag->GetInteractorData().DragTranslationVelocity;
				OtherHandDraggedTo = OtherInteractorThatWasAssistingDrag->GetInteractorData().LastDragToLocation + OtherHandDragDelta;
				OtherHandDragDeltaFromStart = OtherHandDraggedTo - OtherInteractorThatWasAssistingDrag->GetInteractorData().ImpactLocationAtDragStart;
			}


			FVector LaserPointerStart = InteractorData.Transform.GetLocation();
			FVector LaserPointerDirection = InteractorData.Transform.GetUnitAxis( EAxis::X );
			FVector LaserPointerEnd = InteractorData.Transform.GetLocation();

			if( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
				InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
				InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
			{
				// Move objects using the laser pointer (in world space)
				if( InteractorData.bDraggingWithGrabberSphere )
				{
					FSphere GrabberSphere;
					if( Interactor->GetGrabberSphere( /* Out */ GrabberSphere ) )
					{
						// @todo grabber:  Fill in	LaserPointerStart	LaserPointerEnd		LaserPointerDirection   (not used for anything when in TransformablesFreely mode)

						const FVector RotatedOffsetFromSphereCenterAtDragStart = InteractorData.ImpactLocationAtDragStart - InteractorData.GrabberSphereLocationAtDragStart;
						const FVector UnrotatedOffsetFromSphereCenterAtDragStart = InteractorData.InteractorTransformAtDragStart.GetRotation().UnrotateVector( RotatedOffsetFromSphereCenterAtDragStart );
						DraggedTo = GrabberSphere.Center + InteractorData.Transform.GetRotation().RotateVector( UnrotatedOffsetFromSphereCenterAtDragStart );

						const FVector WorldSpaceDragDelta = DraggedTo - InteractorData.LastDragToLocation;
						DragDelta = WorldSpaceDragDelta;
						InteractorData.DragTranslationVelocity = WorldSpaceDragDelta;

						const FVector WorldSpaceDeltaFromStart = DraggedTo - InteractorData.ImpactLocationAtDragStart;
						DragDeltaFromStart = WorldSpaceDeltaFromStart;

						InteractorData.LastDragToLocation = DraggedTo;

						// Update hover location (we only do this when dragging using the laser pointer)
						Interactor->SetHoverLocation(DraggedTo);
					}
					else
					{
						// We lost our grabber sphere, so cancel the drag
						StopDragging( Interactor );
					}
				}
				else if( Interactor->GetLaserPointer( /* Out */ LaserPointerStart, /* Out */ LaserPointerEnd ) )
				{
					LaserPointerDirection = ( LaserPointerEnd - LaserPointerStart ).GetSafeNormal();

					if( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
					{
						// Check to see if the laser pointer is over something we can drop on
						FVector HitLocation = FVector::ZeroVector;
						bool bHitSomething = FindPlacementPointUnderLaser( Interactor, /* Out */ HitLocation );
						if( !bHitSomething )
						{
							HitLocation = LaserPointerStart + LaserPointerDirection * InteractorData.DragRayLength;
						}

						if( InteractorData.bIsFirstDragUpdate || !bFreezePlacementWhileInterpolatingTransformables || !bIsInterpolatingTransformablesFromSnapshotTransform )
						{
							InteractorData.DragRayLength = ( LaserPointerStart - HitLocation ).Size();
							DraggedTo = HitLocation;

							// If the object moved reasonably far between frames, it might be because the angle we were aligning
							// the object with during placement changed radically.  To avoid it popping, we smoothly interpolate
							// it's position over a very short timespan
							if( !bIsInterpolatingTransformablesFromSnapshotTransform )	// Let the last animation finish first
							{
								const FVector WorldSpaceDragDelta = InteractorData.bIsFirstDragUpdate ? FVector::ZeroVector : ( DraggedTo - InteractorData.LastDragToLocation );
								const double ScaledDragDistance = WorldSpaceDragDelta.Size() * WorldScaleFactor;
								if( ScaledDragDistance >= VI::DragAtLaserImpactInterpolationThreshold->GetFloat() )
								{
									bIsInterpolatingTransformablesFromSnapshotTransform = true;
									bFreezePlacementWhileInterpolatingTransformables = true;
									TransformablesInterpolationStartTime = CurrentTime;
									TransformablesInterpolationDuration = VI::DragAtLaserImpactInterpolationDuration->GetFloat();

									// Snapshot time!
									InteractorData.GizmoInterpolationSnapshotTransform = InteractorData.GizmoLastTransform;
								}
							}
						}
						else
						{
							// Keep interpolating toward the previous placement location
							check( bFreezePlacementWhileInterpolatingTransformables );
							DraggedTo = InteractorData.LastDragToLocation;
						}
					}
					else
					{
						DraggedTo = LaserPointerStart + LaserPointerDirection * InteractorData.DragRayLength;
					}

					const FVector WorldSpaceDragDelta = DraggedTo - InteractorData.LastDragToLocation;
					DragDelta = WorldSpaceDragDelta;
					InteractorData.DragTranslationVelocity = WorldSpaceDragDelta;

					const FVector WorldSpaceDeltaFromStart = DraggedTo - InteractorData.ImpactLocationAtDragStart;
					DragDeltaFromStart = WorldSpaceDeltaFromStart;

					InteractorData.LastDragToLocation = DraggedTo;

					// Update hover location (we only do this when dragging using the laser pointer)
					Interactor->SetHoverLocation(FMath::ClosestPointOnLine(LaserPointerStart, LaserPointerEnd, DraggedTo));
				}
				else
				{
					// We lost our laser pointer, so cancel the drag
					StopDragging( Interactor );
				}

				if( OtherInteractor != nullptr )	// @todo grabber: Second hand support (should be doable)
				{
					FVector OtherHandLaserPointerStart, OtherHandLaserPointerEnd;
					if( OtherInteractor->GetLaserPointer( /* Out */ OtherHandLaserPointerStart, /* Out */ OtherHandLaserPointerEnd ) )
					{
						FViewportInteractorData& OtherInteractorData = OtherInteractor->GetInteractorData();

						const FVector OtherHandLaserPointerDirection = ( OtherHandLaserPointerEnd - OtherHandLaserPointerStart ).GetSafeNormal();
						OtherHandDraggedTo = OtherHandLaserPointerStart + OtherHandLaserPointerDirection * OtherInteractorData.DragRayLength;

						const FVector OtherHandWorldSpaceDragDelta = OtherHandDraggedTo - OtherInteractorData.LastDragToLocation;
						OtherHandDragDelta = OtherHandWorldSpaceDragDelta;
						OtherInteractorData.DragTranslationVelocity = OtherHandWorldSpaceDragDelta;

						const FVector OtherHandWorldSpaceDeltaFromStart = OtherHandDraggedTo - OtherInteractorData.ImpactLocationAtDragStart;
						OtherHandDragDeltaFromStart = OtherHandWorldSpaceDeltaFromStart;

						OtherInteractorData.LastDragToLocation = OtherHandDraggedTo;

						// Only hover if we're using the laser pointer
						OtherInteractor->SetHoverLocation(OtherHandDraggedTo);
					}
					else
					{
						// We lost our laser pointer, so cancel the drag assist
						StopDragging( OtherInteractor );
					}
				}
			}
			else if( ensure( InteractorData.DraggingMode == EViewportInteractionDraggingMode::World ) )
			{
				// While we're changing WorldToMetersScale every frame, our room-space hand locations will be scaled as well!  We need to
				// inverse compensate for this scaling so that we can figure out how much the hands actually moved as if no scale happened.
				// This only really matters when we're performing world scaling interactively.
				const FVector RoomSpaceUnscaledHandLocation = ( InteractorData.RoomSpaceTransform.GetLocation() / WorldToMetersScale ) * LastWorldToMetersScale;
				const FVector RoomSpaceUnscaledHandDelta = ( RoomSpaceUnscaledHandLocation - InteractorData.LastRoomSpaceTransform.GetLocation() );

				// Move the world (in room space)
				DraggedTo = InteractorData.LastRoomSpaceTransform.GetLocation() + RoomSpaceUnscaledHandDelta;

				InteractorData.DragTranslationVelocity = RoomSpaceUnscaledHandDelta;
				DragDelta = RoomSpaceUnscaledHandDelta;

				InteractorData.LastDragToLocation = DraggedTo;

				// Two handed?
				if( OtherInteractor != nullptr )
				{
					FViewportInteractorData& OtherInteractorData = OtherInteractor->GetInteractorData();

					const FVector OtherHandRoomSpaceUnscaledLocation = ( OtherInteractorData.RoomSpaceTransform.GetLocation() / WorldToMetersScale ) * LastWorldToMetersScale;
					const FVector OtherHandRoomSpaceUnscaledHandDelta = ( OtherHandRoomSpaceUnscaledLocation - OtherInteractorData.LastRoomSpaceTransform.GetLocation() );

					OtherHandDraggedTo = OtherInteractorData.LastRoomSpaceTransform.GetLocation() + OtherHandRoomSpaceUnscaledHandDelta;

					OtherInteractorData.DragTranslationVelocity = OtherHandRoomSpaceUnscaledHandDelta;
					OtherHandDragDelta = OtherHandRoomSpaceUnscaledHandDelta;

					OtherInteractorData.LastDragToLocation = OtherHandDraggedTo;
				}
			}

			{
				const bool bIsMouseCursorInteractor = Cast<UMouseCursorInteractor>( Interactor ) != nullptr;
				{
					// Don't bother with inertia if we're not moving very fast.  This filters out tiny accidental movements.
					const FVector RoomSpaceHandDelta = bIsMouseCursorInteractor ?
						( DragDelta ) :	// For the mouse cursor the interactor origin won't change unless the camera moves, so just test the distance we dragged instead.
						( InteractorData.RoomSpaceTransform.GetLocation() - InteractorData.LastRoomSpaceTransform.GetLocation() );
					if( RoomSpaceHandDelta.Size() < VI::MinVelocityForInertia->GetFloat() * WorldScaleFactor )
					{
						InteractorData.DragTranslationVelocity = FVector::ZeroVector;
					}
					if( AssistingDragWithInteractor != nullptr )
					{
						FViewportInteractorData& AssistingOtherInteractorData = AssistingDragWithInteractor->GetInteractorData();
						const FVector OtherHandRoomSpaceHandDelta = ( AssistingOtherInteractorData.RoomSpaceTransform.GetLocation() - AssistingOtherInteractorData.LastRoomSpaceTransform.GetLocation() );
						if( OtherHandRoomSpaceHandDelta.Size() < VI::MinVelocityForInertia->GetFloat() * WorldScaleFactor )
						{
							AssistingOtherInteractorData.DragTranslationVelocity = FVector::ZeroVector;
						}
					}
				}
			}

			UViewportDragOperation* DragOperation = nullptr;
			if (InteractorData.DragOperationComponent.IsValid())
			{
				UViewportDragOperationComponent* DragOperationComponent = InteractorData.DragOperationComponent.Get();
				if (DragOperationComponent->GetDragOperation() == nullptr)
				{
					DragOperationComponent->StartDragOperation();
				}
				DragOperation = DragOperationComponent->GetDragOperation();
			}

			const FVector OldViewLocation = DefaultOptionalViewportClient != nullptr ? DefaultOptionalViewportClient->GetViewLocation() : FVector::ZeroVector;

			// Dragging transform gizmo handle
			const bool bWithTwoHands = ( OtherInteractor != nullptr || OtherInteractorThatWasAssistingDrag != nullptr );
			const bool bIsLaserPointerValid = true;
			FVector UnsnappedDraggedTo = FVector::ZeroVector;
			UpdateDragging(
				DeltaTime,
				InteractorData.bIsFirstDragUpdate, 
				Interactor,
				InteractorData.DraggingMode, 
				DragOperation,
				bWithTwoHands,
				InteractorData.OptionalHandlePlacement, 
				DragDelta, 
				OtherHandDragDelta,
				DraggedTo,
				OtherHandDraggedTo,
				DragDeltaFromStart, 
				OtherHandDragDeltaFromStart,
				LaserPointerStart, 
				LaserPointerDirection, 
				Interactor->GetLaserPointerMaxLength(),
				bIsLaserPointerValid, 
				InteractorData.GizmoStartTransform, 
				InteractorData.GizmoLastTransform,
				InteractorData.GizmoTargetTransform,
				InteractorData.GizmoUnsnappedTargetTransform,
				InteractorData.GizmoInterpolationSnapshotTransform,
				InteractorData.GizmoStartLocalBounds,
				InteractorData.DraggingTransformGizmoComponent.Get(),
				/* In/Out */ InteractorData.GizmoSpaceFirstDragUpdateOffsetAlongAxis,
				/* In/Out */ InteractorData.GizmoSpaceDragDeltaFromStartOffset,
				/* In/Out */ InteractorData.LockedWorldDragMode,
				/* In/Out */ InteractorData.GizmoScaleSinceDragStarted,
				/* In/Out */ InteractorData.GizmoRotationRadiansSinceDragStarted,
				InteractorData.bIsDrivingVelocityOfSimulatedTransformables,
				/* Out */ UnsnappedDraggedTo );


			// Make sure the hover point is right on the position that we're dragging the object to.  This is important
			// when constraining dragged objects to a single axis or a plane
			if( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo )
			{
				InteractorData.HoverLocation = FMath::ClosestPointOnSegment( UnsnappedDraggedTo, LaserPointerStart, LaserPointerEnd );
			}

			if( OtherInteractorThatWasAssistingDrag != nullptr )
			{
				OtherInteractorThatWasAssistingDrag->GetInteractorData().LastDragToLocation = OtherHandDraggedTo;

				// Apply damping
				const bool bVelocitySensitive = InteractorData.DraggingMode == EViewportInteractionDraggingMode::World;
				ApplyVelocityDamping( OtherInteractorThatWasAssistingDrag->GetInteractorData().DragTranslationVelocity, bVelocitySensitive );
			}

			// If we were dragging the world, then play some haptic feedback
			if( InteractorData.DraggingMode == EViewportInteractionDraggingMode::World )
			{
				const FVector NewViewLocation = DefaultOptionalViewportClient != nullptr ? DefaultOptionalViewportClient->GetViewLocation() : FVector::ZeroVector;

				// @todo vreditor: Consider doing this for inertial moves too (we need to remember the last hand that invoked the move.)

				const float RoomSpaceHapticTranslationInterval = 25.0f;		// @todo vreditor tweak: Hard coded haptic distance
				const float WorldSpaceHapticTranslationInterval = RoomSpaceHapticTranslationInterval * WorldScaleFactor;

				bool bCrossedGridLine = false;
				for( int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex )
				{
					const int32 OldGridCellIndex = FMath::TruncToInt32( OldViewLocation[ AxisIndex ] / WorldSpaceHapticTranslationInterval );
					const int32 NewGridCellIndex = FMath::TruncToInt32( NewViewLocation[ AxisIndex ] / WorldSpaceHapticTranslationInterval );
					if( OldGridCellIndex != NewGridCellIndex )
					{
						bCrossedGridLine = true;
					}
				}

				if( bCrossedGridLine )
				{				
					// @todo vreditor: Make this a velocity-sensitive strength?
					const float ForceFeedbackStrength = VI::GridHapticFeedbackStrength->GetFloat();		// @todo vreditor tweak: Force feedback strength and enable/disable should be user configurable in options
					Interactor->PlayHapticEffect( ForceFeedbackStrength );
					if ( bWithTwoHands )
					{
						Interactor->GetOtherInteractor()->PlayHapticEffect( ForceFeedbackStrength );
					}
				}
			}
		}
		else if ( InteractorData.DraggingMode == EViewportInteractionDraggingMode::Interactable )
		{
			if ( DraggedInteractable )
			{
				UViewportDragOperationComponent* DragOperationComponent = DraggedInteractable->GetDragOperationComponent();
				if ( DragOperationComponent )
				{
					UViewportDragOperation* DragOperation = DragOperationComponent->GetDragOperation();
					if ( DragOperation )
					{
						DragOperation->ExecuteDrag( Interactor, DraggedInteractable );
					}
				}
			}
			else
			{
				InteractorData.DraggingMode = EViewportInteractionDraggingMode::Nothing;
			}
		}
		// If we're not actively dragging, apply inertia to any selected elements that we've dragged around recently
		else 
		{
			if( (!InteractorData.DragTranslationVelocity.IsNearlyZero( VI::DragTranslationVelocityStopEpsilon->GetFloat() ) ||	bIsInterpolatingTransformablesFromSnapshotTransform) &&
				!InteractorData.bWasAssistingDrag && 	// If we were only assisting, let the other hand take care of doing the update
				!InteractorData.bIsDrivingVelocityOfSimulatedTransformables )	// If simulation mode is on, let the physics engine take care of inertia
			{
				if( InteractorData.LastDraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
					InteractorData.LastDraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
					InteractorData.LastDraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
				{
					InertiaFromInteractor = Interactor;
				}

				const FVector DragDelta = InteractorData.DragTranslationVelocity;
				const FVector DraggedTo = InteractorData.LastDragToLocation + DragDelta;
				const FVector DragDeltaFromStart = DraggedTo - InteractorData.ImpactLocationAtDragStart;

				UViewportInteractor* OtherInteractorThatWasAssistingDrag = GetOtherInteractorIntertiaContribute( Interactor );

				const bool bWithTwoHands = ( OtherInteractorThatWasAssistingDrag != nullptr );

				FVector OtherHandDragDelta = FVector::ZeroVector;
				FVector OtherHandDragDeltaFromStart = FVector::ZeroVector;
				FVector OtherHandDraggedTo = FVector::ZeroVector;
				if( bWithTwoHands )
				{
					FViewportInteractorData& OtherInteractorThatWasAssistingDragData = OtherInteractorThatWasAssistingDrag->GetInteractorData();
					OtherHandDragDelta = OtherInteractorThatWasAssistingDragData.DragTranslationVelocity;
					OtherHandDraggedTo = OtherInteractorThatWasAssistingDragData.LastDragToLocation + OtherHandDragDelta;
					OtherHandDragDeltaFromStart = OtherHandDraggedTo - OtherInteractorThatWasAssistingDragData.ImpactLocationAtDragStart;
				}

				const bool bIsLaserPointerValid = false;	// Laser pointer has moved on to other things
				FVector UnsnappedDraggedTo = FVector::ZeroVector;
				UpdateDragging(
					DeltaTime,
					InteractorData.bIsFirstDragUpdate, 
					Interactor,
					InteractorData.LastDraggingMode, 
					InteractorData.LastDragOperation,
					bWithTwoHands, 
					InteractorData.OptionalHandlePlacement, 
					DragDelta, 
					OtherHandDragDelta,
					DraggedTo,
					OtherHandDraggedTo,
					DragDeltaFromStart, 
					OtherHandDragDeltaFromStart,
					FVector::ZeroVector,	// No valid laser pointer during inertia
					FVector::ZeroVector,	// No valid laser pointer during inertia
					0.0f,		// No valid laser pointer during inertia
					bIsLaserPointerValid, 
					InteractorData.GizmoStartTransform, 
					InteractorData.GizmoLastTransform,
					InteractorData.GizmoTargetTransform,
					InteractorData.GizmoUnsnappedTargetTransform,
					InteractorData.GizmoInterpolationSnapshotTransform,
					InteractorData.GizmoStartLocalBounds,
					InteractorData.DraggingTransformGizmoComponent.Get(),
					/* In/Out */ InteractorData.GizmoSpaceFirstDragUpdateOffsetAlongAxis,
					/* In/Out */ InteractorData.GizmoSpaceDragDeltaFromStartOffset,
					/* In/Out */ InteractorData.LockedWorldDragMode,
					/* In/Out */ InteractorData.GizmoScaleSinceDragStarted,
					/* In/Out */ InteractorData.GizmoRotationRadiansSinceDragStarted,
					InteractorData.bIsDrivingVelocityOfSimulatedTransformables,
					/* Out */ UnsnappedDraggedTo );

				InteractorData.LastDragToLocation = DraggedTo;
				const bool bVelocitySensitive = InteractorData.LastDraggingMode == EViewportInteractionDraggingMode::World;
				ApplyVelocityDamping( InteractorData.DragTranslationVelocity, bVelocitySensitive );

				if( OtherInteractorThatWasAssistingDrag != nullptr )
				{
					FViewportInteractorData& OtherInteractorThatWasAssistingDragData = OtherInteractorThatWasAssistingDrag->GetInteractorData();
					OtherInteractorThatWasAssistingDragData.LastDragToLocation = OtherHandDraggedTo;
					ApplyVelocityDamping( OtherInteractorThatWasAssistingDragData.DragTranslationVelocity, bVelocitySensitive );
				}
			}
			else
			{
				InteractorData.DragTranslationVelocity = FVector::ZeroVector;
			}
		}
	}


	// Update transformables
	const bool bSmoothSnappingEnabled = IsSmoothSnappingEnabled();
	if ( bAreTransformablesMoving && ( bSmoothSnappingEnabled || bIsInterpolatingTransformablesFromSnapshotTransform ) )
	{
		const float SmoothSnapSpeed = VI::SmoothSnapSpeed->GetFloat();
		const bool bUseElasticSnapping = bSmoothSnappingEnabled && 
										VI::ElasticSnap->GetInt() > 0 && 
										DraggingWithInteractor != nullptr &&	// Only while we're still dragging stuff!
										VI::ActorSnap->GetInt() == 0;	// Not while using actor align/snap
		const float ElasticSnapStrength = VI::ElasticSnapStrength->GetFloat();

		float InterpProgress = 1.0f;
		const bool bWasInterpolationNeeded = bIsInterpolatingTransformablesFromSnapshotTransform;
		bool bDidInterpolationFinishThisUpdate = false;
		if( bIsInterpolatingTransformablesFromSnapshotTransform )
		{
			InterpProgress = FMath::Clamp( (float)( CurrentTime - TransformablesInterpolationStartTime ).GetTotalSeconds() / TransformablesInterpolationDuration, 0.0f, 1.0f );
			if( InterpProgress >= 1.0f - KINDA_SMALL_NUMBER )
			{
				// Finished interpolating
				bIsInterpolatingTransformablesFromSnapshotTransform = false;
				bFreezePlacementWhileInterpolatingTransformables = false;
				bDidInterpolationFinishThisUpdate = true;
			}
		}

		bool bIsStillSmoothSnapping = false;

		UViewportInteractor* TransformingInteractor = DraggingWithInteractor != nullptr ? DraggingWithInteractor : InertiaFromInteractor;
		if( TransformingInteractor != nullptr )
		{
			FViewportInteractorData& InteractorData = TransformingInteractor->GetInteractorData();

			FTransform ActualGizmoTargetTransform = InteractorData.GizmoTargetTransform;

			// If 'elastic snapping' is turned on, we'll have the object 'reach' toward its unsnapped position, so
			// that the user always has visual feedback when they are dragging something around, even if they
			// haven't dragged far enough to overcome the snap threshold.
			if( bUseElasticSnapping )
			{
				ActualGizmoTargetTransform.BlendWith( InteractorData.GizmoUnsnappedTargetTransform, ElasticSnapStrength );
			}

			FTransform InterpolatedGizmoTransform = ActualGizmoTargetTransform;
			if( !ActualGizmoTargetTransform.Equals( InteractorData.GizmoLastTransform, 0.0f ) )
			{
				// If we're really close, just snap it.
				if( ActualGizmoTargetTransform.Equals( InteractorData.GizmoLastTransform, KINDA_SMALL_NUMBER ) )
				{
					InterpolatedGizmoTransform = InteractorData.GizmoLastTransform = ActualGizmoTargetTransform;
				}
				else
				{
					bIsStillSmoothSnapping = true;

					InterpolatedGizmoTransform.Blend(
						InteractorData.GizmoLastTransform,
						ActualGizmoTargetTransform,
						FMath::Min( 1.0f, SmoothSnapSpeed * FMath::Min( DeltaTime, 1.0f / 30.0f ) ) );
				}
				InteractorData.GizmoLastTransform = InterpolatedGizmoTransform;
			}

			if( bIsInterpolatingTransformablesFromSnapshotTransform )
			{
				InterpolatedGizmoTransform.BlendWith( InteractorData.GizmoInterpolationSnapshotTransform, 1.0f - InterpProgress );
			}
			
			for ( TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables )
			{
				FViewportTransformable& Transformable = *TransformablePtr;

				// Update the transform!
				// NOTE: We never sweep while smooth-snapping as the object will never end up where we want it to
				// due to friction with the ground and other objects.
				const bool bSweep = false;
				Transformable.ApplyTransform( Transformable.StartTransform * InteractorData.GizmoStartTransform.Inverse() * InterpolatedGizmoTransform, bSweep );
			}
		}

		// Did we finish interpolating in this update?  If so, we need to finalize things and notify everyone
		if( bAreTransformablesMoving &&
			DraggingWithInteractor == nullptr &&
			InertiaFromInteractor == nullptr &&
			( !bWasInterpolationNeeded || bDidInterpolationFinishThisUpdate ) && 
			!bIsStillSmoothSnapping )
		{
			FinishedMovingTransformables();
		}
	}
	else
	{
		// Neither smooth snapping or interpolation is enabled right now, but the transformables could have some velocity
		// applied.  We'll check to see whether they've come to a rest, and if so finalize their positions.
		if( bAreTransformablesMoving && 
			DraggingWithInteractor == nullptr &&
			InertiaFromInteractor == nullptr )
		{
			FinishedMovingTransformables();
		}
	}


	// Refresh the transform gizmo every frame, just in case actors were moved by some external
	// influence.  Also, some features of the transform gizmo respond to camera position (like the
	// measurement text labels), so it just makes sense to keep it up to date.
	{
		const bool bNewObjectsSelected = false;
		RefreshTransformGizmo( bNewObjectsSelected );
	}


	LastWorldToMetersScale = WorldToMetersScale;
} 

bool UViewportWorldInteraction::IsInteractableComponent( const UActorComponent* Component ) const
{
	bool bResult = false;

	// Don't interact primitive components that have been set as not selectable
	if (Component != nullptr)
	{
		const UPrimitiveComponent* ComponentAsPrimitive = Cast<UPrimitiveComponent>(Component);
		if (ComponentAsPrimitive != nullptr)
		{
			bResult = (ComponentAsPrimitive->bSelectable == true);
		}
	}
	
	return bResult;
}

ABaseTransformGizmo* UViewportWorldInteraction::GetTransformGizmoActor()
{
	SpawnTransformGizmoIfNeeded();
	return TransformGizmoActor;
}

void UViewportWorldInteraction::SetDraggedSinceLastSelection( const bool bNewDraggedSinceLastSelection )
{
	bDraggedSinceLastSelection = bNewDraggedSinceLastSelection;
}

void UViewportWorldInteraction::SetLastDragGizmoStartTransform( const FTransform NewLastDragGizmoStartTransform )
{
	LastDragGizmoStartTransform = NewLastDragGizmoStartTransform;
}

const TArray<UViewportInteractor*>& UViewportWorldInteraction::GetInteractors() const
{
	return Interactors;
}

BEGIN_FUNCTION_BUILD_OPTIMIZATION
void UViewportWorldInteraction::UpdateDragging(
	const float DeltaTime,
	bool& bIsFirstDragUpdate,
	UViewportInteractor* Interactor,
	const EViewportInteractionDraggingMode DraggingMode,
	UViewportDragOperation* DragOperation,
	const bool bWithTwoHands,
	const TOptional<FTransformGizmoHandlePlacement> OptionalHandlePlacement,
	const FVector& DragDelta,
	const FVector& OtherHandDragDelta,
	const FVector& DraggedTo,
	const FVector& OtherHandDraggedTo,
	const FVector& DragDeltaFromStart,
	const FVector& OtherHandDragDeltaFromStart,
	const FVector& LaserPointerStart,
	const FVector& LaserPointerDirection,
	const float LaserPointerMaxLength,
	const bool bIsLaserPointerValid,
	const FTransform& GizmoStartTransform,
	FTransform& GizmoLastTransform,
	FTransform& GizmoTargetTransform,
	FTransform& GizmoUnsnappedTargetTransform,
	const FTransform& GizmoInterpolationSnapshotTransform,
	const FBox& GizmoStartLocalBounds,
	const USceneComponent* const DraggingTransformGizmoComponent,
	FVector& GizmoSpaceFirstDragUpdateOffsetAlongAxis,
	FVector& DragDeltaFromStartOffset,
	ELockedWorldDragMode& LockedWorldDragMode,
	float& GizmoScaleSinceDragStarted,
	float& GizmoRotationRadiansSinceDragStarted,
	bool& bIsDrivingVelocityOfSimulatedTransformables,
	FVector& OutUnsnappedDraggedTo )
{
	// IMPORTANT NOTE: Depending on the DraggingMode, the incoming coordinates can be in different coordinate spaces.
	//		-> When dragging objects around, everything is in world space unless otherwise specified in the variable name.
	//		-> When dragging the world around, everything is in room space unless otherwise specified.

	bool bMovedTransformGizmo = false;

	// This will be set to true if we want to set the physics velocity on the dragged objects based on how far
	// we dragged this frame.  Used for pushing objects around in Simulate mode.  If this is set to false, we'll
	// simply zero out the velocities instead to cancel out the physics engine's applied forces (e.g. gravity) 
	// while we're dragging objects around (otherwise, the objects will spaz out as soon as dragging stops.)
	bool bShouldApplyVelocitiesFromDrag = false;

	const ECoordSystem CoordSystem = GetTransformGizmoCoordinateSpace();

	// Always snap objects relative to where they were when we first grabbed them.  We never want objects to immediately
	// teleport to the closest snap, but instead snaps should always be relative to the gizmo center
	// NOTE: While placing objects, we always snap in world space, since the initial position isn't really at all useful
	const bool bLocalSpaceSnapping =
		CoordSystem == COORD_Local &&
		DraggingMode != EViewportInteractionDraggingMode::TransformablesAtLaserImpact;
	const FVector SnapGridBase = ( DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact ||  bLocalSpaceSnapping ) ? FVector::ZeroVector : GizmoStartTransform.GetLocation();

	// Okay, time to move stuff!  We'll do this part differently depending on whether we're dragging actual actors around
	// or we're moving the camera (aka. dragging the world)
	if (DragOperation != nullptr && DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo)
	{

		FVector OutClosestPointOnLaser;
		FVector ConstrainedDragDeltaFromStart = ComputeConstrainedDragDeltaFromStart(
			bIsFirstDragUpdate,
			DragOperation->bPlaneConstraint,
			OptionalHandlePlacement,
			DragDeltaFromStart,
			LaserPointerStart,
			LaserPointerDirection, 
			bIsLaserPointerValid,
			GizmoStartTransform,
			LaserPointerMaxLength,
			/* In/Out */ GizmoSpaceFirstDragUpdateOffsetAlongAxis,
			/* In/Out */ DragDeltaFromStartOffset,
			/* Out */ OutClosestPointOnLaser);

		// Set out put for hover point
		OutUnsnappedDraggedTo = GizmoStartTransform.GetLocation() + OutClosestPointOnLaser;

		// Grid snap!
		const FVector DesiredGizmoLocation = GizmoStartTransform.GetLocation() + ConstrainedDragDeltaFromStart;

		FDraggingTransformableData DragData;
		DragData.Interactor = Interactor;
		DragData.WorldInteraction = this;
		DragData.OptionalHandlePlacement = OptionalHandlePlacement;
		DragData.DragDelta = DragDelta;
		DragData.ConstrainedDragDelta = ConstrainedDragDeltaFromStart;
		DragData.OtherHandDragDelta = OtherHandDragDelta;
		DragData.DraggedTo = DraggedTo;
		DragData.OtherHandDraggedTo = OtherHandDraggedTo;
		DragData.DragDeltaFromStart = DragDeltaFromStart;
		DragData.OtherHandDragDeltaFromStart = OtherHandDragDeltaFromStart;
		DragData.LaserPointerStart = LaserPointerStart;
		DragData.LaserPointerDirection = LaserPointerDirection;
		DragData.GizmoStartTransform = GizmoStartTransform;
		DragData.GizmoLastTransform = GizmoLastTransform;
		DragData.OutGizmoUnsnappedTargetTransform = GizmoUnsnappedTargetTransform;
		DragData.OutUnsnappedDraggedTo = OutUnsnappedDraggedTo;
		DragData.GizmoStartLocalBounds = GizmoStartLocalBounds;
		DragData.GizmoCoordinateSpace = GetTransformGizmoCoordinateSpace();
		DragData.PassDraggedTo = DesiredGizmoLocation;

		DragOperation->ExecuteDrag(DragData);

		GizmoTargetTransform = GizmoUnsnappedTargetTransform;
		if (DragData.bAllowSnap)
		{
			// Translation snap
			if (DragData.bOutTranslated && (AreAligningToActors() || FSnappingUtils::IsSnapToGridEnabled()))
			{
				const FVector ResultLocation = SnapLocation(bLocalSpaceSnapping,
					DragData.OutGizmoUnsnappedTargetTransform.GetLocation(),
					GizmoStartTransform, 
					SnapGridBase, 
					true /*bShouldConstrainMovement*/, 
					ConstrainedDragDeltaFromStart);
				GizmoTargetTransform.SetLocation(ResultLocation);
			}

			// Rotation snap
			if (DragData.bOutRotated && FSnappingUtils::IsRotationSnapEnabled())
			{
				FRotator RotationToSnap = DragData.OutGizmoUnsnappedTargetTransform.GetRotation().Rotator();
				FSnappingUtils::SnapRotatorToGrid(RotationToSnap);
				GizmoTargetTransform.SetRotation(RotationToSnap.Quaternion());
			}

			// Scale snap
			if (DragData.bOutScaled && FSnappingUtils::IsScaleSnapEnabled())
			{
				FVector ScaleToSnap = DragData.OutGizmoUnsnappedTargetTransform.GetScale3D();
				FSnappingUtils::SnapScale(ScaleToSnap, SnapGridBase);
				GizmoTargetTransform.SetScale3D(ScaleToSnap);
			}
		}

		GizmoUnsnappedTargetTransform = DragData.OutGizmoUnsnappedTargetTransform;
		bMovedTransformGizmo = DragData.bOutMovedTransformGizmo;
		bShouldApplyVelocitiesFromDrag = DragData.bOutShouldApplyVelocitiesFromDrag;
	}
	else if (DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact)
	{
		FVector OutClosestPointOnLaser;
		FVector ConstrainedDragDeltaFromStart = ComputeConstrainedDragDeltaFromStart(
			bIsFirstDragUpdate,
			false,
			OptionalHandlePlacement,
			DragDeltaFromStart,
			LaserPointerStart,
			LaserPointerDirection,
			bIsLaserPointerValid,
			GizmoStartTransform,
			LaserPointerMaxLength,
			/* In/Out */ GizmoSpaceFirstDragUpdateOffsetAlongAxis,
			/* In/Out */ DragDeltaFromStartOffset,
			/* Out */ OutClosestPointOnLaser);

		const FVector DesiredGizmoLocation = GizmoStartTransform.GetLocation() + ConstrainedDragDeltaFromStart;

		// Set out put for hover point
		OutUnsnappedDraggedTo = GizmoStartTransform.GetLocation() + OutClosestPointOnLaser;

		// Grid snap!
		const bool bShouldConstrainMovement = true;
		FVector SnappedDraggedTo = SnapLocation(bLocalSpaceSnapping, DesiredGizmoLocation, GizmoStartTransform, SnapGridBase, bShouldConstrainMovement, ConstrainedDragDeltaFromStart);

		// Two passes.  First update the real transform.  Then update the unsnapped transform.
		for (int32 PassIndex = 0; PassIndex < 2; ++PassIndex)
		{
			const bool bIsUpdatingUnsnappedTarget = (PassIndex == 1);
			const FVector& PassDraggedTo = bIsUpdatingUnsnappedTarget ? DesiredGizmoLocation : SnappedDraggedTo;

			// Translate the gizmo!
			FTransform& PassGizmoTargetTransform = bIsUpdatingUnsnappedTarget ? GizmoUnsnappedTargetTransform : GizmoTargetTransform;
			PassGizmoTargetTransform.SetLocation(PassDraggedTo);

			bMovedTransformGizmo = true;
			bShouldApplyVelocitiesFromDrag = true;
		}
	}
	else if ( DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
			  DraggingMode == EViewportInteractionDraggingMode::World )
	{
		// If we have only one thing selected and it's a certain type of object (like a camera), then we might be able to "carry" it instead of
		// translating it like usual.  Carrying just means to freely rotate and translate the object proportionally to how far the end of the
		// laser has moved since you started dragging it.  It's useful for things that you are aiming, like a camera.
		const bool bCanBeCarried = Transformables.Num() == 1 && Transformables[0].Get()->ShouldBeCarried();
		const bool bIsCarrying = bCanBeCarried && VI::AllowCarryingCertainObjects->GetInt() != 0 && DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely && !bWithTwoHands;

		FTransform NewGizmoToWorld = FTransform::Identity;
		if (bIsCarrying)
		{
			const FTransform InteractorToWorld = Interactor->GetInteractorData().Transform;
			const FTransform WorldToInteractor = InteractorToWorld.Inverse();
			const FTransform LaserImpactToInteractor(FRotator::ZeroRotator, WorldToInteractor.TransformPosition(DraggedTo));
			const FTransform LaserImpactToWorld = LaserImpactToInteractor * InteractorToWorld;

			const FTransform LaserImpactStartToInteractorStart = LaserImpactToInteractor;	// @todo: Should use DraggedToAtStart, not DraggedTo above for this ideally (won't handle laser length change during drag)

			const FTransform InteractorStartToWorld = Interactor->GetInteractorData().InteractorTransformAtDragStart;
			const FTransform LaserImpactStartToWorld = LaserImpactStartToInteractorStart * InteractorStartToWorld;

			const FTransform GizmoStartToWorld = GizmoStartTransform;
			const FTransform WorldToGizmoStart = GizmoStartToWorld.Inverse();
			const FTransform LaserImpactStartToGizmoStart = LaserImpactStartToWorld * WorldToGizmoStart;
			const FTransform FixedOffsetFromLaser = LaserImpactStartToGizmoStart.Inverse();

			const FTransform NewTargetTransform = FixedOffsetFromLaser * LaserImpactToWorld;

			if (!LastCarryingTransform.IsSet())
			{
				LastCarryingTransform = NewTargetTransform;
			}

			// Apply smoothing
			NewGizmoToWorld.Blend(LastCarryingTransform.GetValue(), NewTargetTransform, VI::CarrySmoothingLerpAlpha->GetFloat());

			LastCarryingTransform = NewGizmoToWorld;
		}
		else
		{
			FVector TranslationOffset = FVector::ZeroVector;
			FQuat RotationOffset = FQuat::Identity;
			FVector ScaleOffset = FVector(1.0f);

			bool bHasPivotLocation = false;
			FVector PivotLocation = FVector::ZeroVector;

			if ( bWithTwoHands )
			{
				// Rotate/scale while simultaneously counter-translating (two hands)

				// NOTE: The reason that we use per-frame deltas (rather than the delta from the start of the whole interaction,
				// like we do with the other interaction modes), is because the point that we're rotating/scaling relative to
				// actually changes every update as the user moves their hands.  So it's iteratively getting the user to
				// the result they naturally expect.

				const FVector LineStart = DraggedTo;
				const FVector LineStartDelta = DragDelta;
				const double LineStartDistance = LineStartDelta.Size();
				const FVector LastLineStart = LineStart - LineStartDelta;

				const FVector LineEnd = OtherHandDraggedTo;
				const FVector LineEndDelta = OtherHandDragDelta;
				const double LineEndDistance = LineEndDelta.Size();
				const FVector LastLineEnd = LineEnd - LineEndDelta;

				// Choose a point along the new line segment to rotate and scale relative to.  We'll weight it toward the
				// side of the line that moved the most this update.
				const double TotalDistance = LineStartDistance + LineEndDistance;
				double LineStartToEndActivityWeight = 0.5f;	// Default to right in the center, if no distance moved yet.
				if (GetDefault<UVISettings>()->bScaleWorldWithDynamicPivot && !FMath::IsNearlyZero( TotalDistance ) )	// Avoid division by zero
				{
					LineStartToEndActivityWeight = LineStartDistance / TotalDistance;
				}

				PivotLocation = FMath::Lerp( LastLineStart, LastLineEnd, LineStartToEndActivityWeight );
				bHasPivotLocation = true;

				if ( DraggingMode == EViewportInteractionDraggingMode::World && GetDefault<UVISettings>()->bScaleWorldFromFloor)
				{
					PivotLocation.Z = 0.0f;
				}

				// @todo vreditor debug
				#if 0
				{
					const FTransform LinesRelativeTo = DraggingMode == EViewportInteractionDraggingMode::World ? GetRoomTransform() : FTransform::Identity;
					DrawDebugLine( GetWorld(), LinesRelativeTo.TransformPosition( LineStart ), LinesRelativeTo.TransformPosition( LineEnd ), FColor::Green, false, 0.0f );
					DrawDebugLine( GetWorld(), LinesRelativeTo.TransformPosition( LastLineStart ), LinesRelativeTo.TransformPosition( LastLineEnd ), FColor::Red, false, 0.0f );

					DrawDebugSphere( GetWorld(), LinesRelativeTo.TransformPosition( PivotLocation ), 2.5f * GetWorldScaleFactor(), 32, FColor::White, false, 0.0f );
				}
				#endif

				const double LastLineLength = ( LastLineEnd - LastLineStart ).Size();
				const double LineLength = ( LineEnd - LineStart ).Size();
				ScaleOffset = FVector( LineLength / LastLineLength );
				//			ScaleOffset = FVector( 0.98f + 0.04f * FMath::MakePulsatingValue( FPlatformTime::Seconds(), 0.1f ) ); // FVector( LineLength / LastLineLength );
				GizmoScaleSinceDragStarted += FloatCastChecked<float>((LineLength - LastLineLength) / GetWorldScaleFactor(), 1.0 / 16.0);

				// How much did the line rotate since last time?
				RotationOffset = FQuat::FindBetweenVectors( LastLineEnd - LastLineStart, LineEnd - LineStart );
				GizmoRotationRadiansSinceDragStarted += (float)RotationOffset.AngularDistance( FQuat::Identity );

				// For translation, only move proportionally to the common vector between the two deltas.  Basically,
				// you need to move both hands in the same direction to translate while gripping with two hands.
				const FVector AverageDelta = ( DragDelta + OtherHandDragDelta ) * 0.5f;
				const double TranslationWeight = FMath::Max( 0.0f, FVector::DotProduct( DragDelta.GetSafeNormal(), OtherHandDragDelta.GetSafeNormal() ) );
				TranslationOffset = FMath::Lerp( FVector::ZeroVector, AverageDelta, TranslationWeight );
			}
			else
			{
				// Translate only (one hand)
				TranslationOffset = DragDelta;
				//Apply drag scale only if dragging world
				if (DraggingMode == EViewportInteractionDraggingMode::World)
				{
					TranslationOffset *= VI::DragScale->GetFloat();
				}
				GizmoScaleSinceDragStarted = 0.0f;
				GizmoRotationRadiansSinceDragStarted = 0.0f;
			}

			if ( VI::AllowVerticalWorldMovement->GetInt() == 0 )
			{
				TranslationOffset.Z = 0.0f;
			}

			if ( DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely )
			{
				if( !bHasPivotLocation )
				{
					PivotLocation = GizmoUnsnappedTargetTransform.GetLocation();
					bHasPivotLocation = true;
				}

				{
					const FTransform ScaleOffsetTransform( FQuat::Identity, FVector::ZeroVector, ScaleOffset );
					const FTransform TranslationOffsetTransform( FQuat::Identity, TranslationOffset );
					const FTransform RotationOffsetTransform( RotationOffset, FVector::ZeroVector );

					const FTransform PivotToWorld = FTransform( FQuat::Identity, PivotLocation );
					const FTransform WorldToPivot = FTransform( FQuat::Identity, -PivotLocation );
					NewGizmoToWorld = GizmoUnsnappedTargetTransform * WorldToPivot * ScaleOffsetTransform * RotationOffsetTransform * PivotToWorld * TranslationOffsetTransform;
				}
			}
			else if ( DraggingMode == EViewportInteractionDraggingMode::World && !bSkipInteractiveWorldMovementThisFrame )
			{
				FTransform RoomTransform = GetRoomTransform();

				if( bWithTwoHands )
				{
					if(!GetDefault<UVISettings>()->bAllowSimultaneousWorldScalingAndRotation &&
						LockedWorldDragMode == ELockedWorldDragMode::Unlocked )
					{
					const bool bHasDraggedEnoughToScale = FMath::Abs(GizmoScaleSinceDragStarted) >= VI::WorldScalingDragThreshold->GetFloat();
					if (bHasDraggedEnoughToScale)
					{
						LockedWorldDragMode = ELockedWorldDragMode::OnlyScaling;
					}

					const bool bHasDraggedEnoughToRotate = FMath::Abs(GizmoRotationRadiansSinceDragStarted) >= FMath::DegreesToRadians(VI::WorldRotationDragThreshold->GetFloat());
					if (bHasDraggedEnoughToRotate)
					{
						LockedWorldDragMode = ELockedWorldDragMode::OnlyRotating;
					}
					}
				}
				else
				{
					// Only one hand is dragging world, so make sure everything is unlocked
					LockedWorldDragMode = ELockedWorldDragMode::Unlocked;
					GizmoScaleSinceDragStarted = 0.0f;
					GizmoRotationRadiansSinceDragStarted = 0.0f;
				}

				const bool bAllowWorldTranslation =
					(LockedWorldDragMode == ELockedWorldDragMode::Unlocked);

				const bool bAllowWorldScaling =
					bWithTwoHands
					&&
					(
					(
						(GetDefault<UVISettings>()->bAllowSimultaneousWorldScalingAndRotation)
						&&
						(LockedWorldDragMode == ELockedWorldDragMode::Unlocked)
						)
						||
						(LockedWorldDragMode == ELockedWorldDragMode::OnlyScaling)
						);

				const bool bAllowWorldRotation =
					bWithTwoHands
					&&
					(
					(
						(GetDefault<UVISettings>()->bAllowSimultaneousWorldScalingAndRotation)
						&&
						(LockedWorldDragMode == ELockedWorldDragMode::Unlocked)
						)
						||
						(LockedWorldDragMode == ELockedWorldDragMode::OnlyRotating)
						);

				if (bAllowWorldScaling)
				{
					// Adjust world scale
					const float WorldScaleOffset = (float)ScaleOffset.GetAbsMax();
					if (WorldScaleOffset != 0.0f)
					{
						const float OldWorldToMetersScale = GetWorld()->GetWorldSettings()->WorldToMeters;
						const float NewWorldToMetersScale = OldWorldToMetersScale / WorldScaleOffset;

						// NOTE: Instead of clamping, we simply skip changing the W2M this frame if it's out of bounds.  Clamping makes our math more complicated.
						if (!FMath::IsNearlyEqual(NewWorldToMetersScale, OldWorldToMetersScale) &&
							NewWorldToMetersScale >= GetMinScale() && NewWorldToMetersScale <= GetMaxScale())
						{
							SetWorldToMetersScale(NewWorldToMetersScale);
							CompensateRoomTransformForWorldScale(RoomTransform, NewWorldToMetersScale, PivotLocation);
						}
					}
				}

				// Apply rotation and translation
				{
					FTransform RotationOffsetTransform = FTransform::Identity;
					if (bAllowWorldRotation)
					{
						RotationOffsetTransform.SetRotation(RotationOffset);
						if (VI::AllowWorldRotationPitchAndRoll->GetInt() == 0)
						{
							// Eliminate pitch and roll in rotation offset.  We don't want the user to get sick!
							FRotator YawRotationOffset = RotationOffset.Rotator();
							YawRotationOffset.Pitch = YawRotationOffset.Roll = 0.0f;
							RotationOffsetTransform.SetRotation(YawRotationOffset.Quaternion());
						}
					}

					// Move the camera in the opposite direction, so it feels to the user as if they're dragging the entire world around
					FTransform TranslationOffsetTransform(FTransform::Identity);
					if (bAllowWorldTranslation)
					{
						TranslationOffsetTransform.SetLocation(TranslationOffset);
					}
					const FTransform PivotToWorld = FTransform(FQuat::Identity, PivotLocation) * RoomTransform;
					const FTransform WorldToPivot = PivotToWorld.Inverse();
					RoomTransform = TranslationOffsetTransform.Inverse() * RoomTransform * WorldToPivot * RotationOffsetTransform.Inverse() * PivotToWorld;
				}

				RoomTransformToSetOnFrame = MakeTuple(RoomTransform, CurrentTickNumber + 1);
			}
		}

		if (DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely)
		{
			// Grid snap!
			FTransform SnappedNewGizmoToWorld = NewGizmoToWorld;
			{
				if (FSnappingUtils::IsSnapToGridEnabled() || AreAligningToActors())
				{
					const FVector GizmoDragDelta = NewGizmoToWorld.GetLocation() - GizmoUnsnappedTargetTransform.GetLocation();
					const bool bShouldConstrainMovement = false;
					SnappedNewGizmoToWorld.SetLocation(SnapLocation(bLocalSpaceSnapping, NewGizmoToWorld.GetLocation(), GizmoUnsnappedTargetTransform, SnapGridBase, bShouldConstrainMovement, GizmoDragDelta));
				}

				// Rotation snap
				if (FSnappingUtils::IsRotationSnapEnabled())
				{
					FRotator RotationToSnap = SnappedNewGizmoToWorld.GetRotation().Rotator();
					FSnappingUtils::SnapRotatorToGrid(RotationToSnap);
					SnappedNewGizmoToWorld.SetRotation(RotationToSnap.Quaternion());
				}

				// Scale snap
				if (FSnappingUtils::IsScaleSnapEnabled())
				{
					FVector ScaleToSnap = SnappedNewGizmoToWorld.GetScale3D();
					FSnappingUtils::SnapScale(ScaleToSnap, SnapGridBase);
					SnappedNewGizmoToWorld.SetScale3D(ScaleToSnap);
				}
			}

			// Two passes.  First update the real transform.  Then update the unsnapped transform.
			for (int32 PassIndex = 0; PassIndex < 2; ++PassIndex)
			{
				const bool bIsUpdatingUnsnappedTarget = (PassIndex == 1);

				const FTransform NewTransform = (bIsUpdatingUnsnappedTarget ? NewGizmoToWorld : SnappedNewGizmoToWorld);

				FTransform& PassGizmoTargetTransform = bIsUpdatingUnsnappedTarget ? GizmoUnsnappedTargetTransform : GizmoTargetTransform;
				PassGizmoTargetTransform = NewTransform;
				bMovedTransformGizmo = true;
				bShouldApplyVelocitiesFromDrag = true;
			}
		}
	}


	// If we're not using smooth snapping, go ahead and immediately update the transforms of all objects.  If smooth
	// snapping is enabled, this will be done in Tick() instead.
	if (bMovedTransformGizmo)
	{
		// Update velocity if we're simulating in editor
		if (GEditor->bIsSimulatingInEditor)
		{
			FVector MoveDelta = GizmoUnsnappedTargetTransform.GetLocation() - GizmoLastTransform.GetLocation();
			if( bShouldApplyVelocitiesFromDrag )
			{
				bIsDrivingVelocityOfSimulatedTransformables = true;
			}
			else
			{
				MoveDelta = FVector::ZeroVector;
			}

			for (TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables)
			{
				FViewportTransformable& Transformable = *TransformablePtr;

				// @todo VREditor physics: When freely transforming, should set angular velocity too (will pivot be the same though??)
				if (Transformable.IsPhysicallySimulated())
				{
					Transformable.SetLinearVelocity( ( MoveDelta * VI::InertiaVelocityBoost->GetFloat() ) / DeltaTime );
				}
			}
		}

		const bool bSmoothSnappingEnabled = IsSmoothSnappingEnabled();
		if (!bSmoothSnappingEnabled && !bIsInterpolatingTransformablesFromSnapshotTransform)
		{
			GizmoLastTransform = GizmoTargetTransform;
			const bool bSweep = bShouldApplyVelocitiesFromDrag && VI::SweepPhysicsWhileSimulating->GetInt() != 0 && bIsDrivingVelocityOfSimulatedTransformables;
			for (TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables)
			{
				FViewportTransformable& Transformable = *TransformablePtr;

				// Update the transform!
				Transformable.ApplyTransform(Transformable.StartTransform * GizmoStartTransform.Inverse() * GizmoTargetTransform , bSweep);
			}
		}
	}

	bIsFirstDragUpdate = false;
	bSkipInteractiveWorldMovementThisFrame = false;
}
END_FUNCTION_BUILD_OPTIMIZATION

FVector UViewportWorldInteraction::ComputeConstrainedDragDeltaFromStart( 
	const bool bIsFirstDragUpdate, 
	const bool bOnPlane,
	const TOptional<FTransformGizmoHandlePlacement> OptionalHandlePlacement, 
	const FVector& DragDeltaFromStart, 
	const FVector& LaserPointerStart, 
	const FVector& LaserPointerDirection, 
	const bool bIsLaserPointerValid, 
	const FTransform& GizmoStartTransform, 
	const float LaserPointerMaxLength,
	FVector& GizmoSpaceFirstDragUpdateOffsetAlongAxis, 
	FVector& GizmoSpaceDragDeltaFromStartOffset,
	FVector& OutClosestPointOnLaser) const
{
	FVector ConstrainedGizmoSpaceDeltaFromStart;

	bool bConstrainDirectlyToLineOrPlane = false;
	if ( OptionalHandlePlacement.IsSet() )
	{
		// Constrain to line/plane
		const FTransformGizmoHandlePlacement& HandlePlacement = OptionalHandlePlacement.GetValue();

		int32 CenterHandleCount, FacingAxisIndex, CenterAxisIndex;
		HandlePlacement.GetCenterHandleCountAndFacingAxisIndex( /* Out */ CenterHandleCount, /* Out */ FacingAxisIndex, /* Out */ CenterAxisIndex );

		// Our laser pointer ray won't be valid for inertial transform, since it's already moved on to other things.  But the existing velocity should already be on axis.
		bConstrainDirectlyToLineOrPlane = ( CenterHandleCount == 2 ) && bIsLaserPointerValid;
		if ( bConstrainDirectlyToLineOrPlane )
		{
			const FVector GizmoSpaceLaserPointerStart = GizmoStartTransform.InverseTransformPosition( LaserPointerStart );
			const FVector GizmoSpaceLaserPointerDirection = GizmoStartTransform.InverseTransformVectorNoScale( LaserPointerDirection );

			FVector GizmoSpaceConstraintAxis =
				FacingAxisIndex == 0 ? FVector::ForwardVector : ( FacingAxisIndex == 1 ? FVector::RightVector : FVector::UpVector );
			if ( HandlePlacement.Axes[ FacingAxisIndex ] == ETransformGizmoHandleDirection::Negative )
			{
				GizmoSpaceConstraintAxis *= -1.0f;
			}

			if ( bOnPlane )
			{
				const FPlane GizmoSpaceConstrainToPlane( FVector::ZeroVector, GizmoSpaceConstraintAxis );

				// 2D Plane
				FVector GizmoSpacePointOnPlane = FVector::ZeroVector;
				if ( !FMath::SegmentPlaneIntersection(
					GizmoSpaceLaserPointerStart,
					GizmoSpaceLaserPointerStart + LaserPointerMaxLength * GizmoSpaceLaserPointerDirection,
					GizmoSpaceConstrainToPlane,
					/* Out */ GizmoSpacePointOnPlane ) )
				{
					// No intersect.  Default to no delta.
					GizmoSpacePointOnPlane = FVector::ZeroVector;
				}

				// Gizmo space is defined as the starting position of the interaction, so we simply take the closest position
				// along the plane as our delta from the start in gizmo space
				ConstrainedGizmoSpaceDeltaFromStart = GizmoSpacePointOnPlane;

				// Set out for the closest point on laser for clipping
				OutClosestPointOnLaser = GizmoStartTransform.TransformVector(GizmoSpacePointOnPlane);
			}
			else
			{
				// 1D Line
				const float AxisSegmentLength = LaserPointerMaxLength * 2.0f;	// @todo vreditor: We're creating a line segment to represent an infinitely long axis, but really it just needs to be further than our laser pointer can reach

				DVector GizmoSpaceClosestPointOnLaserDouble, GizmoSpaceClosestPointOnAxisDouble;
				SegmentDistToSegmentDouble(
					GizmoSpaceLaserPointerStart, DVector( GizmoSpaceLaserPointerStart ) + DVector( GizmoSpaceLaserPointerDirection ) * LaserPointerMaxLength,
					DVector( GizmoSpaceConstraintAxis ) * -AxisSegmentLength, DVector( GizmoSpaceConstraintAxis ) * AxisSegmentLength,
					/* Out */ GizmoSpaceClosestPointOnLaserDouble,
					/* Out */ GizmoSpaceClosestPointOnAxisDouble );

				// Gizmo space is defined as the starting position of the interaction, so we simply take the closest position
				// along the axis as our delta from the start in gizmo space
				ConstrainedGizmoSpaceDeltaFromStart = GizmoSpaceClosestPointOnAxisDouble.ToFVector();

				// Set out for the closest point on laser for clipping
				OutClosestPointOnLaser = GizmoStartTransform.TransformVector(GizmoSpaceClosestPointOnLaserDouble.ToFVector());
			}

			// Account for the handle position on the outside of the bounds.  This is a bit confusing but very important for dragging
			// to feel right.  When constraining movement to either a plane or single axis, we always want the object to move exactly 
			// to the location under the laser/cursor -- it's an absolute movement.  But if we did that on the first update frame, it 
			// would teleport from underneath the gizmo handle to that location. 
			{
				if ( bIsFirstDragUpdate )
				{
					GizmoSpaceFirstDragUpdateOffsetAlongAxis = ConstrainedGizmoSpaceDeltaFromStart;
				}
				ConstrainedGizmoSpaceDeltaFromStart -= GizmoSpaceFirstDragUpdateOffsetAlongAxis;

				// OK, it gets even more complicated.  When dragging and releasing for inertial movement, this code path
				// will no longer be executed (because the hand/laser has moved on to doing other things, and our drag
				// is driven by velocity only).  So we need to remember the LAST offset from the object's position to
				// where we actually constrained it to, and continue to apply that delta during the inertial drag.
				// That actually happens in the code block near the bottom of this function.
				const FVector GizmoSpaceDragDeltaFromStart = GizmoStartTransform.InverseTransformVectorNoScale( DragDeltaFromStart );
				GizmoSpaceDragDeltaFromStartOffset = ConstrainedGizmoSpaceDeltaFromStart - GizmoSpaceDragDeltaFromStart;
			}
		}
	}

	if ( !bConstrainDirectlyToLineOrPlane )
	{
		ConstrainedGizmoSpaceDeltaFromStart = GizmoStartTransform.InverseTransformVectorNoScale( DragDeltaFromStart );

		// Apply axis constraint if we have one (and we haven't already constrained directly to a line)
		if ( OptionalHandlePlacement.IsSet() )
		{
			// OK in this case, inertia is moving our selected objects while they remain constrained to
			// either a plane or a single axis.  Every frame, we need to apply the original delta between
			// where the laser was aiming and where the object was constrained to on the LAST frame before
			// the user released the object and it moved inertially.  See the big comment up above for 
			// more information.  Inertial movement of constrained objects is actually very complicated!
			ConstrainedGizmoSpaceDeltaFromStart += GizmoSpaceDragDeltaFromStartOffset;

			const FTransformGizmoHandlePlacement& HandlePlacement = OptionalHandlePlacement.GetValue();
			for ( int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex )
			{
				if ( bOnPlane )
				{
					if ( HandlePlacement.Axes[ AxisIndex ] == ETransformGizmoHandleDirection::Positive )
					{
						// Lock the opposing axes out (plane translation)
						ConstrainedGizmoSpaceDeltaFromStart[ AxisIndex ] = 0.0f;
					}
				}
				else
				{
					if ( HandlePlacement.Axes[ AxisIndex ] == ETransformGizmoHandleDirection::Center )
					{
						// Lock the centered axis out (line translation)
						ConstrainedGizmoSpaceDeltaFromStart[ AxisIndex ] = 0.0f;
					}
				}
			}
		}
	}

	FVector ConstrainedWorldSpaceDeltaFromStart = GizmoStartTransform.TransformVectorNoScale( ConstrainedGizmoSpaceDeltaFromStart );
	return ConstrainedWorldSpaceDeltaFromStart;
}

void UViewportWorldInteraction::StartDragging( UViewportInteractor* Interactor, UActorComponent* ClickedTransformGizmoComponent, const FVector& HitLocation, const bool bIsPlacingNewObjects, const bool bAllowInterpolationWhenPlacing, const bool bShouldUseLaserImpactDrag, const bool bStartTransaction, const bool bWithGrabberSphere )
{
	bool bHaveGrabberSphere = false;
	bool bHaveLaserPointer = false;
	FSphere GrabberSphere = FSphere( 0 );
	FVector LaserPointerStart = FVector::ZeroVector;
	FVector LaserPointerEnd = FVector::ZeroVector;

	if( bWithGrabberSphere )
	{
		bHaveGrabberSphere = Interactor->GetGrabberSphere( /* Out */ GrabberSphere );
		check( bHaveGrabberSphere );
	}
	else
	{
		bHaveLaserPointer = Interactor->GetLaserPointer( /* Out */ LaserPointerStart, /* Out */ LaserPointerEnd );
		check( bHaveLaserPointer );
	}

	FViewportInteractorData& InteractorData = Interactor->GetInteractorData();

	if( bStartTransaction )
	{
		// Capture undo state
		TrackingTransaction.TransCount++;
		TrackingTransaction.Begin( LOCTEXT( "MovingObjects", "Move Object" ) );
	}
	else
	{
		// Did you forget to use an FScopedTransaction?  If GUndo was null, then most likely we forgot to wrap this call within an editor transaction.
		// The only exception is in Simulate mode, where Undo is not allowed.
		check( GUndo != nullptr || GEditor == nullptr || GEditor->bIsSimulatingInEditor );
	}
	
	// Suspend actor/component modification during each delta step to avoid recording unnecessary overhead into the transaction buffer
	GEditor->DisableDeltaModification( true );

	// Give the interactor a chance to do something when starting dragging
	Interactor->OnStartDragging( HitLocation, bIsPlacingNewObjects );

	const bool bUsingGizmo = ClickedTransformGizmoComponent != nullptr;
	check( !bUsingGizmo || ( ClickedTransformGizmoComponent->GetOwner() == TransformGizmoActor ) );

	// Start dragging the objects right away!
	InteractorData.DraggingMode = InteractorData.LastDraggingMode = bShouldUseLaserImpactDrag ? EViewportInteractionDraggingMode::TransformablesAtLaserImpact :
		( bUsingGizmo ? EViewportInteractionDraggingMode::TransformablesWithGizmo : EViewportInteractionDraggingMode::TransformablesFreely );

	bAreTransformablesMoving = true;

	// Starting a new drag, so make sure the other hand doesn't think it's assisting us
	if( Interactor->GetOtherInteractor() != nullptr )
	{
		FViewportInteractorData& OtherInteractorData = Interactor->GetOtherInteractor()->GetInteractorData();
		OtherInteractorData.bWasAssistingDrag = false;
	}

	InteractorData.bDraggingWithGrabberSphere = bWithGrabberSphere;
	InteractorData.bIsFirstDragUpdate = true;
	InteractorData.bWasAssistingDrag = false;
	if( bWithGrabberSphere )
	{
		// NOTE: With the grabber sphere, we don't allow the user to shift the drag ray distance
		InteractorData.DragRayLength = 0.0f;
	}
	else
	{
		InteractorData.DragRayLength = ( HitLocation - LaserPointerStart ).Size();
	}
	InteractorData.LastDragToLocation = HitLocation;
	InteractorData.InteractorTransformAtDragStart = InteractorData.Transform;
	InteractorData.GrabberSphereLocationAtDragStart = bWithGrabberSphere ? GrabberSphere.Center : FVector::ZeroVector;
	InteractorData.ImpactLocationAtDragStart = HitLocation;
	InteractorData.DragTranslationVelocity = FVector::ZeroVector;
	InteractorData.DragRayLengthVelocity = 0.0f;
	InteractorData.bIsDrivingVelocityOfSimulatedTransformables = false;

	// Start dragging the transform gizmo.  Even if the user clicked on the actor itself, we'll use
	// the gizmo to transform it.
	if ( bUsingGizmo )
	{
		InteractorData.DraggingTransformGizmoComponent = Cast<USceneComponent>( ClickedTransformGizmoComponent );
	}
	else
	{
		InteractorData.DraggingTransformGizmoComponent = nullptr;
	}

	if ( TransformGizmoActor != nullptr )
	{
		InteractorData.DragOperationComponent = TransformGizmoActor->GetInteractionType(InteractorData.DraggingTransformGizmoComponent.Get(), InteractorData.OptionalHandlePlacement);
		InteractorData.GizmoStartTransform = TransformGizmoActor->GetTransform();
		InteractorData.GizmoStartLocalBounds = GizmoLocalBounds;
	}
	else
	{
		InteractorData.DragOperationComponent.Reset();
		InteractorData.GizmoStartTransform = FTransform::Identity;
		InteractorData.GizmoStartLocalBounds = FBox(ForceInit);
	}
	InteractorData.GizmoLastTransform = InteractorData.GizmoTargetTransform = InteractorData.GizmoUnsnappedTargetTransform = InteractorData.GizmoInterpolationSnapshotTransform = InteractorData.GizmoStartTransform;
	InteractorData.GizmoSpaceFirstDragUpdateOffsetAlongAxis = FVector::ZeroVector;	// Will be determined on first update
	InteractorData.GizmoSpaceDragDeltaFromStartOffset = FVector::ZeroVector;	// Set every frame while dragging
	InteractorData.GizmoScaleSinceDragStarted = 0.0f;
	InteractorData.GizmoRotationRadiansSinceDragStarted = 0.0f;

	bDraggedSinceLastSelection = true;
	LastDragGizmoStartTransform = InteractorData.GizmoStartTransform;

	// Make sure all of our transformables are reset to their initial transform before we start dragging them
	for( TUniquePtr<FViewportTransformable>& Transformable : Transformables )
	{
		Transformable->StartTransform = Transformable->GetTransform();
	}

	// Calculate the offset between the average location and the hit location after setting the transformables for the new selected objects
	InteractorData.StartHitLocationToTransformableCenter = CalculateAverageLocationOfTransformables() - HitLocation;

	// If we're placing actors, start interpolating to their actual location.  This helps smooth everything out when
	// using the laser impact point as the target transform
	if( bIsPlacingNewObjects && bAllowInterpolationWhenPlacing )
	{
		bIsInterpolatingTransformablesFromSnapshotTransform = true;
		bFreezePlacementWhileInterpolatingTransformables = false;	// Always update the placement location when dragging out new objects
		const FTimespan CurrentTime = FTimespan::FromSeconds( FPlatformTime::Seconds() );
		TransformablesInterpolationStartTime = CurrentTime;
		TransformablesInterpolationDuration = VI::PlacementInterpolationDuration->GetFloat();
	}

	// Play a haptic effect when objects are picked up
	Interactor->PlayHapticEffect( Interactor->GetDragHapticFeedbackStrength() );

	if (ViewportTransformer)
	{
		ViewportTransformer->OnStartDragging(Interactor);
	}

	OnStartDraggingEvent.Broadcast( Interactor );
}


void UViewportWorldInteraction::StopDragging( UViewportInteractor* Interactor )
{
	FViewportInteractorData& InteractorData = Interactor->GetInteractorData();
	if ( InteractorData.DraggingMode != EViewportInteractionDraggingMode::Nothing )
	{
		LowSpeedInertiaDamping = VI::LowSpeedInertiaDamping->GetFloat();
		HighSpeedInertiaDamping = VI::HighSpeedInertiaDamping->GetFloat();

		OnStopDraggingEvent.Broadcast( Interactor );

		// If the other hand started dragging after we started, allow that hand to "take over" the drag, so the user
		// doesn't have to click again to continue their action.  Inertial effects of the hand that stopped dragging
		// will still be in effect.
		FViewportInteractorData* OtherInteractorData = nullptr;
		if( Interactor->GetOtherInteractor() != nullptr )
		{
			OtherInteractorData = &Interactor->GetOtherInteractor()->GetInteractorData();
		}
		
		if ( OtherInteractorData != nullptr && OtherInteractorData->DraggingMode == EViewportInteractionDraggingMode::AssistingDrag )
		{
			// The other hand takes over whatever this hand was doing
			OtherInteractorData->DraggingMode = OtherInteractorData->LastDraggingMode = InteractorData.DraggingMode;
			OtherInteractorData->bIsDrivingVelocityOfSimulatedTransformables = InteractorData.bIsDrivingVelocityOfSimulatedTransformables;

			OtherInteractorData->GizmoStartTransform = InteractorData.GizmoStartTransform;
			OtherInteractorData->GizmoLastTransform = InteractorData.GizmoLastTransform;
			OtherInteractorData->GizmoTargetTransform = InteractorData.GizmoTargetTransform;
			OtherInteractorData->GizmoUnsnappedTargetTransform = InteractorData.GizmoUnsnappedTargetTransform;
			OtherInteractorData->GizmoInterpolationSnapshotTransform = InteractorData.GizmoInterpolationSnapshotTransform;
			OtherInteractorData->GizmoStartLocalBounds = InteractorData.GizmoStartLocalBounds;

			OtherInteractorData->LockedWorldDragMode = ELockedWorldDragMode::Unlocked;
			OtherInteractorData->GizmoScaleSinceDragStarted = 0.0f;
			OtherInteractorData->GizmoRotationRadiansSinceDragStarted = 0.0f;

			// The other hand is no longer assisting, as it's now the primary interacting hand.
			OtherInteractorData->bWasAssistingDrag = false;

			// The hand that stopped dragging will be remembered as "assisting" the other hand, so that its
			// inertia will still influence interactions
			InteractorData.bWasAssistingDrag = true;
		}
		else
		{
			if ( InteractorData.DraggingMode == EViewportInteractionDraggingMode::Interactable )
			{
				if ( DraggedInteractable )
				{
					DraggedInteractable->OnDragRelease( Interactor );
					UViewportDragOperationComponent* DragOperationComponent =  DraggedInteractable->GetDragOperationComponent();
					if ( DragOperationComponent )
					{
						DragOperationComponent->ClearDragOperation();
					}
				}
			}
			else if ( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
					  InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
					  InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
			{				
				// Destroy drag operation.
				if (InteractorData.DragOperationComponent != nullptr)
				{
					InteractorData.LastDragOperation = InteractorData.DragOperationComponent->GetDragOperation();
					InteractorData.DragOperationComponent->ClearDragOperation();
				}
				InteractorData.DragOperationComponent.Reset();
				InteractorData.DraggingMode = EViewportInteractionDraggingMode::Nothing;

				// If we're not dragging anything around, check to see if transformables have come to a rest.
				const bool bTransformablesStillMoving =
					bAreTransformablesMoving &&
					( IsSmoothSnappingEnabled() ||
					  !InteractorData.DragTranslationVelocity.IsNearlyZero( VI::DragTranslationVelocityStopEpsilon->GetFloat() ) ||
					  bIsInterpolatingTransformablesFromSnapshotTransform );
				if( !bTransformablesStillMoving )
				{
					// Finalize the drag
					FinishedMovingTransformables();
				}	

				if (ViewportTransformer)
				{
					ViewportTransformer->OnStopDragging(Interactor);
				}
			}
		}

		// Make sure we reset the dragging state if we released
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		if (SlateApplication.IsDragDropping())
		{
			TSharedPtr<FGenericWindow> Window = SlateApplication.GetActiveTopLevelWindow()->GetNativeWindow();
			SlateApplication.OnDragLeave(Window);
		}

		InteractorData.DraggingMode = EViewportInteractionDraggingMode::Nothing;
	}
}
	
void UViewportWorldInteraction::FinishedMovingTransformables()
{
	bAreTransformablesMoving = false;
	bIsInterpolatingTransformablesFromSnapshotTransform = false;
	bFreezePlacementWhileInterpolatingTransformables = false;
	TransformablesInterpolationStartTime = FTimespan::Zero();
	TransformablesInterpolationDuration = 1.0f;
	LastCarryingTransform.Reset();

	OnFinishedMovingTransformablesEvent.Broadcast();

	// Finalize undo
	{
		// @todo vreditor undo: This doesn't actually encapsulate any inertial movement that happens after the drag is released! 
		// We need to figure out whether that matters or not.  Also, look into PostEditMove( bFinished=true ) and how that relates to this.
		--TrackingTransaction.TransCount;
		TrackingTransaction.End();
		GEditor->DisableDeltaModification( false );
	}
}


void UViewportWorldInteraction::SetUseInputPreprocessor( bool bInUseInputPreprocessor )
{
	bUseInputPreprocessor = bInUseInputPreprocessor;
}

void UViewportWorldInteraction::AllowWorldMovement(bool bAllow)
{
	bAllowWorldMovement = bAllow;
}

bool UViewportWorldInteraction::FindPlacementPointUnderLaser( UViewportInteractor* Interactor, FVector& OutHitLocation )
{
	// Check to see if the laser pointer is over something we can drop on
	bool bHitSomething = false;
	FVector HitLocation = FVector::ZeroVector;

	bool bAnyPhysicallySimulatedTransformables = false;
	static TArray<AActor*> IgnoredActors;
	{
		IgnoredActors.Reset();

		// Don't trace against stuff that we're dragging!
		for( TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables )
		{
			FViewportTransformable& Transformable = *TransformablePtr;

			Transformable.UpdateIgnoredActorList( IgnoredActors );

			if( Transformable.IsPhysicallySimulated() )
			{
				bAnyPhysicallySimulatedTransformables = true;
			}
		} 
	}


	const EHitResultGizmoFilterMode GizmoFilterMode = EHitResultGizmoFilterMode::NoGizmos; // Never place on top of gizmos, just ignore them
	const bool bEvenIfUIIsInFront = true;	// Don't let the UI block placement
	FHitResult HitResult = Interactor->GetHitResultFromLaserPointer( &IgnoredActors, GizmoFilterMode, nullptr, bEvenIfUIIsInFront );
	if( HitResult.HitObjectHandle.IsValid() )
	{
		bHitSomething = true;
		HitLocation = HitResult.ImpactPoint;
	}

	if( bHitSomething )
	{
		FViewportInteractorData& InteractorData = Interactor->GetInteractorData();

		// Pull back from the impact point
		{
			const FVector GizmoSpaceImpactNormal = InteractorData.GizmoStartTransform.InverseTransformVectorNoScale( HitResult.ImpactNormal );

			FVector ExtremePointOnBox;
			const FVector ExtremeDirection = -GizmoSpaceImpactNormal;
			const FBox& Box = GizmoLocalBounds;
			{
				const FVector ExtremePoint(
					ExtremeDirection.X >= 0.0f ? Box.Max.X : Box.Min.X,
					ExtremeDirection.Y >= 0.0f ? Box.Max.Y : Box.Min.Y,
					ExtremeDirection.Z >= 0.0f ? Box.Max.Z : Box.Min.Z );
				const double ProjectionDistance = FVector::DotProduct( ExtremePoint, ExtremeDirection );

				const double ExtraOffset =
					( bAnyPhysicallySimulatedTransformables && GEditor->bIsSimulatingInEditor ) ? ( GizmoLocalBounds.GetSize().GetAbsMax() * VI::PlacementOffsetScaleWhileSimulating->GetFloat() ) : 0.0f;

				ExtremePointOnBox = ExtremeDirection * ( ProjectionDistance + ExtraOffset );
			}

			const FVector WorldSpaceExtremePointOnBox = InteractorData.GizmoStartTransform.TransformVectorNoScale( ExtremePointOnBox );

			HitLocation -= WorldSpaceExtremePointOnBox;
			HitLocation -= InteractorData.StartHitLocationToTransformableCenter;
		}

		OutHitLocation = HitLocation;
	}

	return bHitSomething;
}

FTrackingTransaction& UViewportWorldInteraction::GetTrackingTransaction()
{
	return TrackingTransaction;
}

bool UViewportWorldInteraction::IsSmoothSnappingEnabled() const
{
	// @todo vreditor perf: We could cache this, perhaps.  There are a few other places we do similar checks. O(N).
	bool bAnyPhysicallySimulatedTransformables = false;
	for( const TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables )
	{
		FViewportTransformable& Transformable = *TransformablePtr;
		if( Transformable.IsPhysicallySimulated() )
		{
			bAnyPhysicallySimulatedTransformables = true;
		}
	}

	const float SmoothSnapSpeed = VI::SmoothSnapSpeed->GetFloat();
	const bool bSmoothSnappingEnabled =
		( !GEditor->bIsSimulatingInEditor || !bAnyPhysicallySimulatedTransformables ) &&
		( FSnappingUtils::IsSnapToGridEnabled() || FSnappingUtils::IsRotationSnapEnabled() || FSnappingUtils::IsScaleSnapEnabled() || VI::ActorSnap->GetInt() == 1) &&
		VI::SmoothSnap->GetInt() != 0 &&
		!FMath::IsNearlyZero( SmoothSnapSpeed );

	return bSmoothSnappingEnabled;
}

void UViewportWorldInteraction::PollInputIfNeeded()
{
	if ( LastFrameNumberInputWasPolled != GFrameNumber )
	{
		for ( UViewportInteractor* Interactor : Interactors )
		{
			Interactor->PollInput();
		}

		LastFrameNumberInputWasPolled = GFrameNumber;
	}
}


void UViewportWorldInteraction::RefreshTransformGizmo( const bool bNewObjectsSelected )
{
	if ( IsTransformGizmoVisible() )
	{
		SpawnTransformGizmoIfNeeded();

		// Make sure the gizmo is visible
		const bool bShouldBeVisible = true;
		const bool bPropagateToChildren = true;
		TransformGizmoActor->GetRootComponent()->SetVisibility( bShouldBeVisible, bPropagateToChildren );

		UViewportInteractor* DraggingWithInteractor = nullptr;
		static TArray< UActorComponent* > HoveringOverHandles;
		HoveringOverHandles.Reset();
		for( UViewportInteractor* Interactor : Interactors )
		{
			FViewportInteractorData& InteractorData = Interactor->GetInteractorData();
			if( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
				InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
				InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
			{
				DraggingWithInteractor = Interactor;
			}

			UActorComponent* HoveringOverHandle = InteractorData.HoveringOverTransformGizmoComponent.Get();
			if( HoveringOverHandle != nullptr )
			{
				HoveringOverHandles.Add( HoveringOverHandle );
			}
		}
		const bool bAllHandlesVisible = ( DraggingWithInteractor == nullptr );
		UActorComponent* SingleVisibleHandle = ( DraggingWithInteractor != nullptr ) ? DraggingWithInteractor->GetInteractorData().DraggingTransformGizmoComponent.Get() : nullptr;


		const ECoordSystem CurrentCoordSystem = GetTransformGizmoCoordinateSpace();
		const bool bIsWorldSpaceGizmo = ( CurrentCoordSystem == COORD_World );

		FViewportTransformable& LastTransformable = *Transformables.Last();

		// Use the location and orientation of the last selected object, to be consistent with the regular editor's gizmo
		FTransform GizmoToWorld = LastTransformable.GetTransform();
		GizmoToWorld.RemoveScaling();	// We don't need the pivot itself to be scaled

		if ( bIsWorldSpaceGizmo )
		{
			GizmoToWorld.SetRotation( FQuat::Identity );
		}

		// Create a gizmo-local bounds around all of the selected objects
		FBox GizmoSpaceSelectedObjectsBounds;
		GizmoSpaceSelectedObjectsBounds.Init();
		{

			for ( TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables )
			{
				FViewportTransformable& Transformable = *TransformablePtr;

				// Get the bounding box into the local space of the gizmo
				const FBox GizmoSpaceBounds = Transformable.BuildBoundingBox( GizmoToWorld );
				GizmoSpaceSelectedObjectsBounds += GizmoSpaceBounds;
			}
		}

		// Don't show gizmo rotation and scale handles when we're dealing with a single unoriented
		// point in space (such as a mesh vertex), which can't reasonably be rotated or scaled.
		const bool bHaveSingleUnorientedPoint =
			Transformables.Num() == 1 &&
			Transformables[ 0 ]->IsUnorientedPoint();
		const bool bAllowRotationAndScaleHandles = !bHaveSingleUnorientedPoint;

		if( ( Transformables.Num() > 1 && this->ViewportTransformer->ShouldCenterTransformGizmoPivot() ) ||
			  VI::ForceGizmoPivotToCenterOfObjectsBounds->GetInt() > 0 )
		{
			const FVector GizmoSpaceBoundsCenterLocation = GizmoSpaceSelectedObjectsBounds.GetCenter();
			GizmoToWorld.SetLocation( GizmoToWorld.TransformPosition( GizmoSpaceBoundsCenterLocation ) );
			GizmoSpaceSelectedObjectsBounds = GizmoSpaceSelectedObjectsBounds.ShiftBy( -GizmoSpaceBoundsCenterLocation );
		}

		if ( bNewObjectsSelected )
		{
			TransformGizmoActor->OnNewObjectsSelected();
		}

		const EGizmoHandleTypes CurrentGizmoType = GetCurrentGizmoType();
		const ECoordSystem GizmoCoordinateSpace = GetTransformGizmoCoordinateSpace();

		GizmoLocalBounds = GizmoSpaceSelectedObjectsBounds;

		// If the user has an HMD on, use the head location for gizmo distance-based sizing, otherwise use the active
		// level editing viewport's camera location.
		// @todo gizmo: Ideally the gizmo would be sized separately for every viewport it's visible within
		FVector ViewerLocation = GizmoToWorld.GetLocation();
		if( HaveHeadTransform() )
		{
			ViewerLocation = GetHeadTransform().GetLocation();
		}
		else if (DefaultOptionalViewportClient != nullptr)
		{
			ViewerLocation = DefaultOptionalViewportClient->GetViewLocation();
		}
		else if( GCurrentLevelEditingViewportClient != nullptr )
		{
			ViewerLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
		}

		TransformGizmoActor->UpdateGizmo( 
			CurrentGizmoType,
			GizmoCoordinateSpace, 
			GizmoToWorld, 
			GizmoSpaceSelectedObjectsBounds, 
			ViewerLocation, 
			TransformGizmoScale, 
			bAllHandlesVisible, 
			bAllowRotationAndScaleHandles,
			SingleVisibleHandle,
			HoveringOverHandles, 
			VI::GizmoHandleHoverScale->GetFloat(), 
			VI::GizmoHandleHoverAnimationDuration->GetFloat() );

		// Only draw if snapping is turned on
		SpawnGridMeshActor();
		if ( FSnappingUtils::IsSnapToGridEnabled() )
		{
	        const bool bShouldGridBeVisible = true;
	        const bool bPropagateToGridChildren = true;
	        SnapGridActor->GetRootComponent()->SetVisibility( bShouldGridBeVisible, bPropagateToGridChildren );

			const float GizmoAnimationAlpha = TransformGizmoActor->GetAnimationAlpha();

			// Position the grid snap actor
			const FVector GizmoLocalBoundsBottom( 0.0f, 0.0f, GizmoLocalBounds.Min.Z );
			const float SnapGridZOffset = 0.1f;	// @todo vreditor tweak: Offset the grid position a little bit to avoid z-fighting with objects resting directly on a floor
			const FVector SnapGridLocation = GizmoToWorld.TransformPosition( GizmoLocalBoundsBottom ) + FVector( 0.0f, 0.0f, SnapGridZOffset );
			SnapGridActor->SetActorLocationAndRotation( SnapGridLocation, GizmoToWorld.GetRotation() );

			// Figure out how big to make the snap grid.  We'll use a multiplier of the object's bounding box size (ignoring local 
			// height, because we currently only draw the grid at the bottom of the object.)
			FBox GizmoLocalBoundsFlattened = GizmoLocalBounds;
			GizmoLocalBoundsFlattened.Min.Z = GizmoLocalBoundsFlattened.Max.Z = 0.0f;
			const FBox GizmoWorldBoundsFlattened = GizmoLocalBoundsFlattened.TransformBy( GizmoToWorld );

			// Make sure we're at least as big as the gizmo bounds, but also large enough that you can see at least a few grid cells (depending on your snap size)
			const double SnapGridSize = 
				FMath::Max( GizmoWorldBoundsFlattened.GetSize().GetAbsMax(), GEditor->GetGridSize() * 2.5f ) * VI::SnapGridSize->GetFloat();

			// The mesh is 100x100, so we'll scale appropriately
			const double SnapGridScale = SnapGridSize / 100.0f;
			SnapGridActor->SetActorScale3D( FVector( SnapGridScale ) );

			EViewportInteractionDraggingMode DraggingMode = EViewportInteractionDraggingMode::Nothing;
			FVector GizmoStartLocationWhileDragging = FVector::ZeroVector;
			for ( UViewportInteractor* Interactor : Interactors )
			{
				const FViewportInteractorData& InteractorData = Interactor->GetInteractorData();
				if ( InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
					InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
					InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact )
				{
					DraggingMode = InteractorData.DraggingMode;
					GizmoStartLocationWhileDragging = InteractorData.GizmoStartTransform.GetLocation();
					break;
				}
				else if ( bDraggedSinceLastSelection &&
					( InteractorData.LastDraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
					InteractorData.LastDraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo ||
					InteractorData.LastDraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact ) )
				{
					DraggingMode = InteractorData.LastDraggingMode;
					GizmoStartLocationWhileDragging = LastDragGizmoStartTransform.GetLocation();
				}
			}

			FVector SnapGridCenter = SnapGridLocation;
			if ( DraggingMode != EViewportInteractionDraggingMode::Nothing )
			{
				SnapGridCenter = GizmoStartLocationWhileDragging;
			}

			// Fade the grid in with the transform gizmo
			const float GridAlpha = GizmoAnimationAlpha;

			// Animate the grid a little bit
			const FLinearColor GridColor = FLinearColor::LerpUsingHSV(
				FLinearColor::White,
				FLinearColor::Yellow,
				FMath::MakePulsatingValue( GetTimeSinceEntered().GetTotalSeconds(), 0.5f ) ).CopyWithNewOpacity( GridAlpha );

			const float GridInterval = GEditor->GetGridSize();

			// If the grid size is very small, shrink the size of our lines so that they don't overlap
			float LineWidth = VI::SnapGridLineWidth->GetFloat();
			while ( GridInterval < LineWidth * 3.0f )	// @todo vreditor tweak
			{
				LineWidth *= 0.5f;
			}

			{
				// Update snap grid material state
				UMaterialInstanceDynamic* LocalSnapGridMID = GetSnapGridMID();

				static FName GridColorParameterName( "GridColor" );
				LocalSnapGridMID->SetVectorParameterValue( GridColorParameterName, GridColor );

				static FName GridCenterParameterName( "GridCenter" );
				LocalSnapGridMID->SetVectorParameterValue( GridCenterParameterName, SnapGridCenter );

				static FName GridIntervalParameterName( "GridInterval" );
				LocalSnapGridMID->SetScalarParameterValue( GridIntervalParameterName, GridInterval );

				static FName GridRadiusParameterName( "GridRadius" );
				LocalSnapGridMID->SetScalarParameterValue( GridRadiusParameterName, FloatCastChecked<float>(SnapGridSize * 0.5f, 1./16.) );

				static FName LineWidthParameterName( "LineWidth" );
				LocalSnapGridMID->SetScalarParameterValue( LineWidthParameterName, LineWidth );

				FMatrix GridRotationMatrix = GizmoToWorld.ToMatrixNoScale().Inverse();
				GridRotationMatrix.SetOrigin( FVector::ZeroVector );

				static FName GridRotationMatrixXParameterName( "GridRotationMatrixX" );
				static FName GridRotationMatrixYParameterName( "GridRotationMatrixY" );
				static FName GridRotationMatrixZParameterName( "GridRotationMatrixZ" );
				LocalSnapGridMID->SetVectorParameterValue( GridRotationMatrixXParameterName, GridRotationMatrix.GetScaledAxis( EAxis::X ) );
				LocalSnapGridMID->SetVectorParameterValue( GridRotationMatrixYParameterName, GridRotationMatrix.GetScaledAxis( EAxis::Y ) );
				LocalSnapGridMID->SetVectorParameterValue( GridRotationMatrixZParameterName, GridRotationMatrix.GetScaledAxis( EAxis::Z ) );
			}
		}
		else
		{
			// Grid snap not enabled
	        const bool bShouldGridBeVisible = false;
	        const bool bPropagateToGridChildren = true;
	        SnapGridActor->GetRootComponent()->SetVisibility( bShouldGridBeVisible, bPropagateToGridChildren );
		}

		if (bNewObjectsSelected && bPlayNextRefreshTransformGizmoSound)
		{
			PlaySound(AssetContainer->SelectionChangeSound, TransformGizmoActor->GetActorLocation());
		}

		bPlayNextRefreshTransformGizmoSound = true;
	}
	else
	{
		// Nothing selected or the gizmo was asked to be hidden
		if( TransformGizmoActor != nullptr )
		{
	        const bool bShouldBeVisible = false;
	        const bool bPropagateToChildren = true;
	        TransformGizmoActor->GetRootComponent()->SetVisibility( bShouldBeVisible, bPropagateToChildren );
		}
		GizmoLocalBounds = FBox(ForceInit);

		// Hide the snap actor
		if( SnapGridActor != nullptr )
		{
			SnapGridActor->GetRootComponent()->SetVisibility( false );
		}
	}
}

void UViewportWorldInteraction::SpawnTransformGizmoIfNeeded()
{
	// Check if there no gizmo or if the wrong gizmo is being used
	bool bSpawnNewGizmo = false;
	if ( TransformGizmoActor == nullptr || TransformGizmoActor->GetClass() != TransformGizmoClass )
	{
		bSpawnNewGizmo = true;
	}

	if ( bSpawnNewGizmo )
	{
		// Destroy the previous gizmo
		if ( TransformGizmoActor != nullptr )
		{
			DestroyTransientActor(TransformGizmoActor);
		}

		// Create the correct gizmo
		const bool bWithSceneComponent = false;	// We already have our own scene component
		TransformGizmoActor = CastChecked<ABaseTransformGizmo>(SpawnTransientSceneActor(TransformGizmoClass, TEXT( "PivotTransformGizmo" ), bWithSceneComponent));

		check( TransformGizmoActor != nullptr );
		TransformGizmoActor->SetOwnerWorldInteraction( this );

		if ( !IsTransformGizmoVisible() )
		{
	        const bool bShouldBeVisible = false;
	        const bool bPropagateToChildren = true;
	        TransformGizmoActor->GetRootComponent()->SetVisibility( bShouldBeVisible, bPropagateToChildren );
		}
	}
}

void UViewportWorldInteraction::SetTransformGizmoVisible( const bool bShouldBeVisible )
{
	bShouldTransformGizmoBeVisible = bShouldBeVisible;

	// NOTE: The actual visibility change will be applied the next tick when RefreshTransformGizmo() is called
}

bool UViewportWorldInteraction::ShouldTransformGizmoBeVisible() const
{
	return bShouldTransformGizmoBeVisible;
}

bool UViewportWorldInteraction::IsTransformGizmoVisible() const
{
	return ( Transformables.Num() > 0 && VI::ShowTransformGizmo->GetInt() != 0 && bShouldTransformGizmoBeVisible && !HasTransformableWithVelocityInSimulate() );
}

void UViewportWorldInteraction::SetTransformGizmoScale( const float NewScale )
{
	TransformGizmoScale = NewScale;

	// NOTE: The actual scale change will be applied the next tick when RefreshTransformGizmo() is called
}

float UViewportWorldInteraction::GetTransformGizmoScale() const
{
	return TransformGizmoScale;
}

void UViewportWorldInteraction::ApplyVelocityDamping( FVector& Velocity, const bool bVelocitySensitive )
{
	const double InertialMovementZeroEpsilon = 0.01f;	// @todo vreditor tweak
	if ( !Velocity.IsNearlyZero( InertialMovementZeroEpsilon ) )
	{
		// Apply damping
		if ( bVelocitySensitive )
		{
			const double DampenMultiplierAtLowSpeeds = LowSpeedInertiaDamping;
			const double DampenMultiplierAtHighSpeeds = HighSpeedInertiaDamping;
			const double SpeedForMinimalDamping = 2.5f * GetWorldScaleFactor();	// cm/frame	// @todo vreditor tweak
			const double SpeedBasedDampeningScalar = FMath::Clamp( Velocity.Size(), 0.0f, SpeedForMinimalDamping ) / SpeedForMinimalDamping;	// @todo vreditor: Probably needs a curve applied to this to compensate for our framerate insensitivity
			Velocity = Velocity * FMath::Lerp( DampenMultiplierAtLowSpeeds, DampenMultiplierAtHighSpeeds, SpeedBasedDampeningScalar );	// @todo vreditor: Frame rate sensitive damping.  Make use of delta time!
		}
		else
		{
			Velocity = Velocity * 0.95f;
		}
	}

	if ( Velocity.IsNearlyZero( InertialMovementZeroEpsilon ) )
	{
		Velocity = FVector::ZeroVector;
	}
}

void UViewportWorldInteraction::CycleTransformGizmoCoordinateSpace()
{
	const bool bGetRawValue = true;
	const ECoordSystem CurrentCoordSystem = GetModeTools().GetCoordSystem( bGetRawValue );
	SetTransformGizmoCoordinateSpace( CurrentCoordSystem == COORD_World ? COORD_Local : COORD_World );
}

void UViewportWorldInteraction::SetTransformGizmoCoordinateSpace( const ECoordSystem NewCoordSystem )
{
	// If we are trying to enter world space but are aligning to actors, turn off aligning to actors
	if (NewCoordSystem == COORD_World && AreAligningToActors())
	{
		if (HasCandidatesSelected())
		{
			SetSelectionAsCandidates();
		}
		GUnrealEd->Exec(GetWorld(), TEXT("VI.EnableGuides 0"));
	}

	GetModeTools().SetCoordSystem( NewCoordSystem );
}

ECoordSystem UViewportWorldInteraction::GetTransformGizmoCoordinateSpace() const
{
	if (AreAligningToActors())
	{
		return COORD_Local;
	}

	const bool bGetRawValue = false;
	return GetModeTools().GetCoordSystem(bGetRawValue);
}

float UViewportWorldInteraction::GetMaxScale()
{
	return VI::ScaleMax->GetFloat();
}

float UViewportWorldInteraction::GetMinScale()
{
	return VI::ScaleMin->GetFloat();
}

void UViewportWorldInteraction::SetWorldToMetersScale( const float NewWorldToMetersScale, const bool bCompensateRoomWorldScale  /*= false*/ )
{
	check (NewWorldToMetersScale > 0);

	// @todo vreditor: This is bad because we're clobbering the world settings which will be saved with the map.  Instead we need to 
	// be able to apply an override before the scene view gets it
	ENGINE_API extern float GNewWorldToMetersScale;
	GNewWorldToMetersScale = NewWorldToMetersScale;
	OnWorldScaleChangedEvent.Broadcast(NewWorldToMetersScale);

	if (bCompensateRoomWorldScale)
	{
		const FVector RoomspacePivotLocation = GetRoomSpaceHeadTransform().GetLocation();
		FTransform NewRoomTransform = GetRoomTransform();
		CompensateRoomTransformForWorldScale(NewRoomTransform, NewWorldToMetersScale, RoomspacePivotLocation);
		RoomTransformToSetOnFrame = MakeTuple( NewRoomTransform, CurrentTickNumber + 1 );
	}
}

UViewportInteractor* UViewportWorldInteraction::GetOtherInteractorIntertiaContribute( UViewportInteractor* Interactor )
{
	// Check to see if the other hand has any inertia to contribute
	UViewportInteractor* OtherInteractorThatWasAssistingDrag = nullptr;
	{
		UViewportInteractor* OtherInteractor = Interactor->GetOtherInteractor();
		if( OtherInteractor != nullptr  )
		{
			FViewportInteractorData& OtherHandInteractorData = OtherInteractor->GetInteractorData();

			// If the other hand isn't doing anything, but the last thing it was doing was assisting a drag, then allow it
			// to contribute inertia!
			if ( OtherHandInteractorData.DraggingMode == EViewportInteractionDraggingMode::Nothing && OtherHandInteractorData.bWasAssistingDrag )
			{
				if ( !OtherHandInteractorData.DragTranslationVelocity.IsNearlyZero( VI::DragTranslationVelocityStopEpsilon->GetFloat() ) )
				{
					OtherInteractorThatWasAssistingDrag = OtherInteractor;
				}
			}
		}
	}

	return OtherInteractorThatWasAssistingDrag;
}

void UViewportWorldInteraction::DestroyActors()
{
	if(TransformGizmoActor != nullptr)
	{
		DestroyTransientActor(TransformGizmoActor);
		TransformGizmoActor = nullptr;
	}

	if(SnapGridActor != nullptr)
	{
		DestroyTransientActor(SnapGridActor);
		SnapGridActor = nullptr;
		SnapGridMeshComponent = nullptr;
	}

	if(SnapGridMID != nullptr)
	{
		SnapGridMID->MarkAsGarbage();
		SnapGridMID = nullptr;
	}
}

bool UViewportWorldInteraction::AreAligningToActors() const
{
	return (VI::ActorSnap->GetInt() == 1) ? true : false;
}

bool UViewportWorldInteraction::HasCandidatesSelected()
{
	return CandidateActors.Num() > 0 ? true : false;
}

void UViewportWorldInteraction::SetSelectionAsCandidates()
{
	if (HasCandidatesSelected())
	{
		CandidateActors.Reset();
	}
	else if (VI::ActorSnap->GetInt() == 1)
	{
		TArray<TUniquePtr<FViewportTransformable>> NewTransformables;

		USelection* ActorSelectionSet = GEditor->GetSelectedActors();

		static TArray<UObject*> SelectedActorObjects;
		SelectedActorObjects.Reset();
		ActorSelectionSet->GetSelectedObjects(AActor::StaticClass(), /* Out */ SelectedActorObjects);

		for (UObject* SelectedActorObject : SelectedActorObjects)
		{
			AActor* SelectedActor = CastChecked<AActor>(SelectedActorObject);
			CandidateActors.Add(SelectedActor);
		}

		Deselect();
	}
}

float UViewportWorldInteraction::GetCurrentDeltaTime() const
{
	return CurrentDeltaTime;
}

bool UViewportWorldInteraction::ShouldSuppressExistingCursor() const
{
	if (VI::ForceShowCursor->GetInt() == 1)
	{
		return false;
	}
	else
	{
		return bShouldSuppressCursor;
	}

}

void UViewportWorldInteraction::SetForceCursor(const bool bInShouldForceCursor)
{
	bShouldForceCursor = bInShouldForceCursor;
}

bool UViewportWorldInteraction::ShouldForceCursor() const
{
	return true;
}

const UViewportInteractionAssetContainer& UViewportWorldInteraction::GetAssetContainer() const
{
	return *AssetContainer;
}

const UViewportInteractionAssetContainer* UViewportWorldInteraction::LoadAssetContainer()
{
	UViewportInteractionAssetContainer* AssetContainer = LoadObject<UViewportInteractionAssetContainer>(nullptr, UViewportWorldInteraction::AssetContainerPath);
	checkf(AssetContainer, TEXT("Failed to load ViewportInteractionAssetContainer (%s). See log for reason."), UViewportWorldInteraction::AssetContainerPath);
	return AssetContainer;
}

void UViewportWorldInteraction::PlaySound(USoundBase* SoundBase, const FVector& InWorldLocation, const float InVolume /*= 1.0f*/)
{
	if (IsActive() && GEditor != nullptr && GEditor->CanPlayEditorSound())
	{
		const float Volume = InVolume*VI::SFXMultiplier->GetFloat();
		UGameplayStatics::PlaySoundAtLocation(GetWorld(), SoundBase, InWorldLocation, FRotator::ZeroRotator, Volume);
	}
}

void UViewportWorldInteraction::SetInVR(const bool bInVR)
{
	bIsInVR = bInVR;
	if (bIsInVR)
	{
		VI::NavigationMode->Set(0);
	}
}

bool UViewportWorldInteraction::IsInVR() const
{
	return bIsInVR;
}

EGizmoHandleTypes UViewportWorldInteraction::GetCurrentGizmoType() const
{
	if (GizmoType.IsSet())
	{
		return GizmoType.GetValue();
	}
	else
	{
		switch( GetModeTools().GetWidgetMode() )
		{
			case UE::Widget::WM_TranslateRotateZ:
				return EGizmoHandleTypes::All;

			case UE::Widget::WM_Translate:
				return EGizmoHandleTypes::Translate;

			case UE::Widget::WM_Rotate:
				return EGizmoHandleTypes::Rotate;

			case UE::Widget::WM_Scale:
				return EGizmoHandleTypes::Scale;
		}
	}

	return EGizmoHandleTypes::Translate;
}

void UViewportWorldInteraction::SetGizmoHandleType( const EGizmoHandleTypes InGizmoHandleType )
{
	GizmoType.Reset();

	switch( InGizmoHandleType )
	{
		case EGizmoHandleTypes::All:
			GizmoType = InGizmoHandleType;
			GetModeTools().SetWidgetMode( UE::Widget::WM_Translate );
			break;

		case EGizmoHandleTypes::Translate:
			GetModeTools().SetWidgetMode( UE::Widget::WM_Translate );
			break;

		case EGizmoHandleTypes::Rotate:
			GetModeTools().SetWidgetMode( UE::Widget::WM_Rotate );
			break;

		case EGizmoHandleTypes::Scale:
			GetModeTools().SetWidgetMode( UE::Widget::WM_Scale );
			break;

		default:
			check(0);
	}
}

void UViewportWorldInteraction::SetTransformGizmoClass( const TSubclassOf<ABaseTransformGizmo>& NewTransformGizmoClass )
{
	TransformGizmoClass = NewTransformGizmoClass;
}

void UViewportWorldInteraction::SetDraggedInteractable( IViewportInteractableInterface* InDraggedInteractable, UViewportInteractor* Interactor )
{
	Interactor->SetDraggingMode(EViewportInteractionDraggingMode::Interactable);
	DraggedInteractable = InDraggedInteractable;
	if ( DraggedInteractable && DraggedInteractable->GetDragOperationComponent() )
	{
		DraggedInteractable->GetDragOperationComponent()->StartDragOperation();
	}
}

bool UViewportWorldInteraction::IsOtherInteractorHoveringOverComponent( UViewportInteractor* Interactor, UActorComponent* Component ) const
{
	bool bResult = false;

	if ( Interactor && Component )
	{
		for ( UViewportInteractor* CurrentInteractor : Interactors )
		{
			if ( CurrentInteractor != Interactor && CurrentInteractor->GetLastHoverComponent() == Component )
			{
				bResult = true;
				break;
			}
		}
	}

	return bResult;
}

void UViewportWorldInteraction::SpawnGridMeshActor()
{
	// Snap grid mesh
	if( SnapGridActor == nullptr )
	{
		const bool bWithSceneComponent = false;
		SnapGridActor = SpawnTransientSceneActor<AActor>(TEXT("SnapGrid"), bWithSceneComponent);
		SnapGridActor->bIsEditorOnlyActor = true;
		SnapGridMeshComponent = NewObject<UStaticMeshComponent>( SnapGridActor );
		SnapGridMeshComponent->MarkAsEditorOnlySubobject();
		SnapGridActor->AddOwnedComponent( SnapGridMeshComponent );
		SnapGridActor->SetRootComponent( SnapGridMeshComponent );
		SnapGridMeshComponent->RegisterComponent();

		UStaticMesh* GridMesh = AssetContainer->GridMesh;
		check( GridMesh != nullptr );
		SnapGridMeshComponent->SetStaticMesh( GridMesh );
		SnapGridMeshComponent->SetMobility( EComponentMobility::Movable );
		SnapGridMeshComponent->SetCollisionEnabled( ECollisionEnabled::NoCollision );

		UMaterialInterface* GridMaterial = AssetContainer->GridMaterial;
		check( GridMaterial != nullptr );

		SnapGridMID = UMaterialInstanceDynamic::Create( GridMaterial, GetTransientPackage() );
		check( SnapGridMID != nullptr );
		SnapGridMeshComponent->SetMaterial( 0, SnapGridMID );

		// The grid starts off hidden
		SnapGridMeshComponent->SetVisibility( false );
	}
}

FVector UViewportWorldInteraction::CalculateAverageLocationOfTransformables()
{
	FVector Result = FVector::ZeroVector;

	for( TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables )
	{
		Result += TransformablePtr.Get()->GetTransform().GetLocation();
	}

	Result /= Transformables.Num();

	return Result;
}

FLinearColor UViewportWorldInteraction::GetColor(const EColors Color, const float Multiplier /*= 1.f*/) const
{
	return Colors[ (int32)Color ] * Multiplier;
}


void UViewportWorldInteraction::AddMouseCursorInteractor()
{
	if( ++DefaultMouseCursorInteractorRefCount == 1 )
	{
		// Add a mouse cursor
		DefaultMouseCursorInteractor = NewObject<UMouseCursorInteractor>();
		DefaultMouseCursorInteractor->Init();
		this->AddInteractor( DefaultMouseCursorInteractor );
	}
}

void UViewportWorldInteraction::ReleaseMouseCursorInteractor()
{
	if( --DefaultMouseCursorInteractorRefCount == 0 )
	{
		DefaultMouseCursorInteractorRefCount = 0;

		// Remove mouse cursor
		this->RemoveInteractor( DefaultMouseCursorInteractor );
		DefaultMouseCursorInteractor = nullptr;
	}
	else
	{
		check( DefaultMouseCursorInteractorRefCount >= 0 );
	}
}

FVector UViewportWorldInteraction::FindTransformGizmoAlignPoint(const FTransform& GizmoStartTransform, const FTransform& DesiredGizmoTransform, const bool bShouldConstrainMovement, FVector ConstraintAxes)
{
	struct Local
	{
		static TArray<FVector> FindLocalSnapPoints(const FBox InBox)
		{
			TArray<FVector> PotentialSnapPoints;
			FVector BoxCenter;
			FVector BoxExtents;

			InBox.GetCenterAndExtents(BoxCenter, BoxExtents);

			FVector PotentialSnapPoint = FVector::ZeroVector;

			// Potential snap points are:
			// The center of each face
			for (int32 X = -1; X < 2; ++X)
			{
				PotentialSnapPoint[0] = X*BoxExtents[0];
				PotentialSnapPoints.AddUnique(PotentialSnapPoint);
				// The center of each edge
				for (int32 Y = -1; Y < 2; ++Y)
				{
					PotentialSnapPoint[1] = Y*BoxExtents[1];
					PotentialSnapPoints.AddUnique(PotentialSnapPoint);
					// Each corner
					for (int32 Z = -1; Z < 2; ++Z)
					{
						PotentialSnapPoint[2] = Z*BoxExtents[2];
						PotentialSnapPoints.AddUnique(PotentialSnapPoint);
					}
				}
			}
			return PotentialSnapPoints;
		}
	};

	TArray<FGuideData> PotentialGizmoGuides;
	// Get all the potential snap points on the transform gizmo at the desired location
	const TArray<FVector> DesiredGizmoLocalGizmoSnapPoints = Local::FindLocalSnapPoints(GizmoLocalBounds);
	DrawBoxBrackets(GizmoLocalBounds, DesiredGizmoTransform, FLinearColor::Yellow);

	// Don't let the guide lines be shorter than the local bounding box extent in any direction. 
	// This helps when objects are close together
	FVector GizmoLocalBoundsExtents;
	FVector GizmoLocalBoundsCenter;
	GizmoLocalBounds.GetCenterAndExtents(GizmoLocalBoundsCenter, GizmoLocalBoundsExtents);
	const FVector MinGuideLength = GizmoLocalBoundsExtents - GizmoLocalBoundsCenter;
	
	if(bShouldConstrainMovement)
	{
		ConstraintAxes = GizmoStartTransform.InverseTransformVector(ConstraintAxes);
	}

	// Our snap distances are some percentage of the transform gizmo's dimensions
	const double AdjustedSnapDistance = (VI::ForceSnapDistance->GetFloat() / 100.0f) * 2.0f * (MinGuideLength.GetAbsMax());
	int32 NumberOfMatchesNeeded = 0;
	if (bShouldConstrainMovement)
	{
		for (int32 PointAxis = 0; PointAxis < 3; ++PointAxis)
		{
			if (!FMath::IsNearlyZero(ConstraintAxes[PointAxis], FVector::FReal(0.0001)))
			{
				NumberOfMatchesNeeded++;
			}
		}
	}
	else
	{
		NumberOfMatchesNeeded = 3;
	}

	TArray<const AActor*> UsingCandidateActors;
	if (HasCandidatesSelected())
	{	
		for (const AActor* SelectedCandidateActor : CandidateActors)
		{
			// Don't align to yourself, the entire world, or any actors hidden in the editor
			if (SelectedCandidateActor != nullptr
				&& !SelectedCandidateActor->IsSelected()
				&& SelectedCandidateActor != GetWorld()->GetDefaultBrush()
				&& SelectedCandidateActor->IsHiddenEd() == false
				&& !SelectedCandidateActor->IsEditorOnly()
				&& SelectedCandidateActor->GetRootComponent() != nullptr
				&& !SelectedCandidateActor->GetRootComponent()->IsEditorOnly())
			{
				UsingCandidateActors.Add(SelectedCandidateActor);
				const FBox LocalActorBoundingBox = SelectedCandidateActor->CalculateComponentsBoundingBoxInLocalSpace();
				DrawBoxBrackets(LocalActorBoundingBox, SelectedCandidateActor->GetTransform(), FLinearColor::Blue);
			}
		}
	}
	else
	{
		// Find all possible candidates for alignment
		// TODO: add the world grid
		// TODO: remove anything it might not make sense to align to
		const float CompareDistance = FloatCastChecked<float>(VI::AlignCandidateDistance->GetFloat() * MinGuideLength.GetAbsMax(), 1./16.);
		const FVector Start = DesiredGizmoTransform.GetLocation();
		const FVector End = DesiredGizmoTransform.GetLocation();
		TArray<FOverlapResult> OutOverlaps;
		bool const bHit = GetWorld()->OverlapMultiByChannel(OutOverlaps, Start, FQuat::Identity, ECC_Visibility, FCollisionShape::MakeSphere(CompareDistance));

		if (bHit)
		{
			for (FOverlapResult OverlapResult : OutOverlaps)
			{
				const AActor* PossibleCandidateActor = OverlapResult.OverlapObjectHandle.FetchActor();

				// Don't align to yourself, the entire world, or any actors hidden in the editor
				if (PossibleCandidateActor != nullptr
					&& !PossibleCandidateActor->IsSelected()
					&& PossibleCandidateActor != GetWorld()->GetDefaultBrush()
					&& PossibleCandidateActor->IsHiddenEd() == false
					&& !PossibleCandidateActor->IsEditorOnly()
					&& PossibleCandidateActor->GetRootComponent() != nullptr
					&& !PossibleCandidateActor->GetRootComponent()->IsEditorOnly())
				{
					{
						// Check if our candidate actor is close enough (comparison multiplier * maximum dimension of actor)
						if (
							(DesiredGizmoTransform.GetLocation() - PossibleCandidateActor->GetActorLocation()).GetAbsMin() <= CompareDistance)
						{
							UsingCandidateActors.Add(PossibleCandidateActor);
						}
					}
				}
			}
		}
	}

	for (const AActor* CandidateActor : UsingCandidateActors)
	{
		// Get the gizmo space snap points for the stationary candidate actor
		const FBox LocalActorBox = CandidateActor->CalculateComponentsBoundingBoxInLocalSpace();
		const TArray<FVector> LocalCandidateSnapPoints = Local::FindLocalSnapPoints(LocalActorBox);

		// Set up the initial guide information for X, Y, and Z guide line
		FGuideData InitialGuideHelper;
		InitialGuideHelper.AlignedActor = CandidateActor;
		InitialGuideHelper.LocalOffset = InitialGuideHelper.SnapPoint = InitialGuideHelper.GuideStart = InitialGuideHelper.GuideEnd = FVector::ZeroVector;
		InitialGuideHelper.GuideColor = FLinearColor::Black.ToFColor(/*bSRGB=*/ true);
		InitialGuideHelper.DrawAlpha = 1.0f;
		InitialGuideHelper.GuideLength = 10000000.0f;

		for (const FVector& LocalCandidateSnapPoint : LocalCandidateSnapPoints)
		{
			FVector WorldCandidateSnapPoint = CandidateActor->ActorToWorld().TransformPosition(LocalCandidateSnapPoint);
			FVector DesiredGizmoLocalCandidateSnapPoint = DesiredGizmoTransform.InverseTransformPosition(WorldCandidateSnapPoint);
			// Check it against each moving snap point
			for (const FVector& DesiredGizmoLocalGizmoSnapPoint : DesiredGizmoLocalGizmoSnapPoints)
			{
				FVector WorldGizmoSnapPoint = DesiredGizmoTransform.TransformPosition(DesiredGizmoLocalGizmoSnapPoint);
				int32 NumberOfMatchingAxes = 0;
				FVector DesiredGizmoLocalOffset = FVector::ZeroVector;
				for (int32 PointAxis = 0; PointAxis < 3; ++PointAxis)
				{
					// If we are within the snap distance and can snap along that axis
					if (FMath::Abs(DesiredGizmoLocalCandidateSnapPoint[PointAxis] - DesiredGizmoLocalGizmoSnapPoint[PointAxis]) <= AdjustedSnapDistance &&
						(!FMath::IsNearlyZero(ConstraintAxes[PointAxis], (FVector::FReal)0.0001) ||
							!bShouldConstrainMovement ))
					{
						NumberOfMatchingAxes++;
						DesiredGizmoLocalOffset[PointAxis] = DesiredGizmoLocalCandidateSnapPoint[PointAxis] - DesiredGizmoLocalGizmoSnapPoint[PointAxis];
					}
				}

				if (NumberOfMatchingAxes >= NumberOfMatchesNeeded)
				{
					FVector DesiredGizmoLocalGuideStart = DesiredGizmoLocalGizmoSnapPoint + DesiredGizmoLocalOffset;

					// Transform the goal location into world space
					const FVector WorldGuideStart = DesiredGizmoTransform.TransformPosition( DesiredGizmoLocalGuideStart );

					InitialGuideHelper.LocalOffset = DesiredGizmoLocalOffset;
					InitialGuideHelper.GuideLength = (WorldCandidateSnapPoint - WorldGuideStart).Size();
					InitialGuideHelper.GuideStart = WorldGuideStart;
					InitialGuideHelper.GuideEnd = WorldCandidateSnapPoint;
					InitialGuideHelper.SnapPoint = WorldGizmoSnapPoint;
					InitialGuideHelper.GuideColor = FLinearColor::Yellow.ToFColor(/*bSRGB=*/ true);
					PotentialGizmoGuides.Add(InitialGuideHelper);	
				}
			}
		}
	}

	// Now find the best guide from all available point combinations
	FGuideData AlignedGuide;
	bool bFoundAlignedTransform = false;
	double AlignedDeltaSize = 10000000.0f;
	for (FGuideData PotentialGizmoGuide : PotentialGizmoGuides)
	{	
		const double OffsetSize = PotentialGizmoGuide.LocalOffset.Size();

		// Keep finding the guide with the shortest offset size
		if (OffsetSize < AlignedDeltaSize && !FMath::IsNearlyZero(OffsetSize))
		{
			bFoundAlignedTransform = true;
			AlignedGuide = PotentialGizmoGuide;
			AlignedDeltaSize = OffsetSize;
		}
	}

	FVector LastBestAlignedLocationOffset = FVector::ZeroVector;
	if (bFoundAlignedTransform)
	{
		LastBestAlignedLocationOffset = GizmoStartTransform.InverseTransformPosition(AlignedGuide.GuideStart) - GizmoStartTransform.InverseTransformPosition(AlignedGuide.SnapPoint);
	}

	return LastBestAlignedLocationOffset;
}

void UViewportWorldInteraction::AddActorToExcludeFromHitTests( AActor* ActorToExcludeFromHitTests )
{
	ActorsToExcludeFromHitTest.Add( ActorToExcludeFromHitTests );

	// Remove expired entries
	for( int32 ActorIndex = 0; ActorIndex < ActorsToExcludeFromHitTest.Num(); ++ActorIndex )
	{
		if( !ActorsToExcludeFromHitTest[ ActorIndex ].IsValid() )
		{
			ActorsToExcludeFromHitTest.RemoveAtSwap( ActorIndex-- );
		}
	}
}




void UViewportWorldInteraction::DrawBoxBrackets(const FBox InActor, const FTransform LocalToWorld, const FLinearColor BracketColor)
{
	struct Local
	{
		static void GetBoundingVectors(const FBox LocalBox, FVector& OutVectorMin, FVector& OutVectorMax)
		{
			OutVectorMin = FVector(BIG_NUMBER);
			OutVectorMax = FVector(-BIG_NUMBER);


			// MinVector
			OutVectorMin.X = FMath::Min<double>(LocalBox.Min.X, OutVectorMin.X);
			OutVectorMin.Y = FMath::Min<double>(LocalBox.Min.Y, OutVectorMin.Y);
			OutVectorMin.Z = FMath::Min<double>(LocalBox.Min.Z, OutVectorMin.Z);
			// MaxVector
			OutVectorMax.X = FMath::Max<double>(LocalBox.Max.X, OutVectorMax.X);
			OutVectorMax.Y = FMath::Max<double>(LocalBox.Max.Y, OutVectorMax.Y);
			OutVectorMax.Z = FMath::Max<double>(LocalBox.Max.Z, OutVectorMax.Z);
		}
	};

	const FColor GROUP_COLOR = BracketColor.ToFColor(/*bSRGB=*/ true);

	FVector MinVector;
	FVector MaxVector;
	Local::GetBoundingVectors(InActor, MinVector, MaxVector);

	// Create a bracket offset to determine the length of our corner axises
	const double BracketOffset = FVector::Dist(MinVector, MaxVector) * 0.1f;

	// Calculate bracket corners based on min/max vectors
	TArray<FVector> BracketCorners;

	// Bottom Corners
	BracketCorners.Add(FVector(MinVector.X, MinVector.Y, MinVector.Z));
	BracketCorners.Add(FVector(MinVector.X, MaxVector.Y, MinVector.Z));
	BracketCorners.Add(FVector(MaxVector.X, MaxVector.Y, MinVector.Z));
	BracketCorners.Add(FVector(MaxVector.X, MinVector.Y, MinVector.Z));

	// Top Corners
	BracketCorners.Add(FVector(MinVector.X, MinVector.Y, MaxVector.Z));
	BracketCorners.Add(FVector(MinVector.X, MaxVector.Y, MaxVector.Z));
	BracketCorners.Add(FVector(MaxVector.X, MaxVector.Y, MaxVector.Z));
	BracketCorners.Add(FVector(MaxVector.X, MinVector.Y, MaxVector.Z));

	for (int32 BracketCornerIndex = 0; BracketCornerIndex < BracketCorners.Num(); ++BracketCornerIndex)
	{
		// Direction corner axis should be pointing based on min/max
		const FVector CORNER = BracketCorners[BracketCornerIndex];
		const int32 DIR_X = CORNER.X == MaxVector.X ? -1 : 1;
		const int32 DIR_Y = CORNER.Y == MaxVector.Y ? -1 : 1;
		const int32 DIR_Z = CORNER.Z == MaxVector.Z ? -1 : 1;

		const FVector LocalBracketX = FVector(CORNER.X + (BracketOffset * DIR_X), CORNER.Y, CORNER.Z);
		const FVector LocalBracketY = FVector(CORNER.X, CORNER.Y + (BracketOffset * DIR_Y), CORNER.Z);
		const FVector LocalBracketZ = FVector(CORNER.X, CORNER.Y, CORNER.Z + (BracketOffset * DIR_Z));

		const FVector WorldCorner = LocalToWorld.TransformPosition(CORNER);
		const FVector WorldBracketX = LocalToWorld.TransformPosition(LocalBracketX);
		const FVector WorldBracketY = LocalToWorld.TransformPosition(LocalBracketY);
		const FVector WorldBracketZ = LocalToWorld.TransformPosition(LocalBracketZ);

		DrawDebugLine(GetWorld(), WorldCorner, WorldBracketX, GROUP_COLOR, false, -1.0f, 1, 2.0f);
		DrawDebugLine(GetWorld(), WorldCorner, WorldBracketY, GROUP_COLOR, false, -1.0f, 1, 2.0f);
		DrawDebugLine(GetWorld(), WorldCorner, WorldBracketZ, GROUP_COLOR, false, -1.0f, 1, 2.0f);
	}

}

void UViewportWorldInteraction::CompensateRoomTransformForWorldScale(FTransform& InOutRoomTransform, const float InNewWorldToMetersScale, const FVector& InRoomPivotLocation)
{
	const float OldWorldToMetersScale = GetWorld()->GetWorldSettings()->WorldToMeters;

	// Because the tracking space size has changed, but our head position within that space relative to the origin
	// of the room is the same (before scaling), we need to offset our location within the tracking space to compensate.
	// This makes the user feel like their head and hands remain in the same location.
	const FVector WorldSpacePivotLocation = InOutRoomTransform.TransformPosition(InRoomPivotLocation);
	const FVector NewRoomSpacePivotLocation = (InRoomPivotLocation / OldWorldToMetersScale) * InNewWorldToMetersScale;
	const FVector NewWorldSpacePivotLocation = InOutRoomTransform.TransformPosition(NewRoomSpacePivotLocation);
	const FVector WorldSpacePivotDelta = (NewWorldSpacePivotLocation - WorldSpacePivotLocation);
	const FVector NewWorldSpaceRoomLocation = InOutRoomTransform.GetLocation() - WorldSpacePivotDelta;

	InOutRoomTransform.SetLocation(NewWorldSpaceRoomLocation);
}

bool UViewportWorldInteraction::HasTransformableWithVelocityInSimulate() const
{
	bool bResult = false;

	// Only check if we are in simulate and if the world of this extension is actually the world simulating.
	if (GEditor->bIsSimulatingInEditor && GetWorld() == GEditor->PlayWorld)
	{
		for (const TUniquePtr<FViewportTransformable>& TransformablePtr : Transformables)
		{
			const FViewportTransformable& Transformable = *TransformablePtr;

			if (!Transformable.GetLinearVelocity().IsNearlyZero(1.0f))
			{
				bResult = true;
				break;
			}
		}
	}

	return bResult;
}

FEditorModeTools& UViewportWorldInteraction::GetModeTools() const
{
	return DefaultOptionalViewportClient == nullptr ? GLevelEditorModeTools() : *DefaultOptionalViewportClient->GetModeTools();
}

FVector UViewportWorldInteraction::SnapLocation(const bool bLocalSpaceSnapping, const FVector& DesiredGizmoLocation, const FTransform &GizmoStartTransform, const FVector SnapGridBase, const bool bShouldConstrainMovement, const FVector AlignAxes)
{
	bool bTransformableAlignmentUsed = false;
	FVector SnappedGizmoLocation = DesiredGizmoLocation;

	const FVector GizmoSpaceDesiredGizmoLocation = GizmoStartTransform.InverseTransformPosition( DesiredGizmoLocation );
	
	if ((VI::ActorSnap->GetInt() == 1) && bLocalSpaceSnapping && ViewportTransformer->CanAlignToActors() == true)
	{
		FTransform DesiredGizmoTransform = GizmoStartTransform;
		DesiredGizmoTransform.SetLocation( DesiredGizmoLocation );

		FVector LocationOffset = FindTransformGizmoAlignPoint(GizmoStartTransform, DesiredGizmoTransform, bShouldConstrainMovement, AlignAxes);
		
		if (!LocationOffset.IsZero())
		{
			bTransformableAlignmentUsed = true;
			const FVector GizmoSpaceSnappedGizmoLocation = GizmoSpaceDesiredGizmoLocation + LocationOffset;
			SnappedGizmoLocation = GizmoStartTransform.TransformPosition( GizmoSpaceSnappedGizmoLocation );
		}
	}

	if (FSnappingUtils::IsSnapToGridEnabled() && !bTransformableAlignmentUsed)
	{
		// Snap in local space, if we need to
		if (bLocalSpaceSnapping)
		{
			FVector GizmoSpaceSnappedGizmoLocation = GizmoSpaceDesiredGizmoLocation;
			FSnappingUtils::SnapPointToGrid(GizmoSpaceSnappedGizmoLocation, SnapGridBase);
			SnappedGizmoLocation = GizmoStartTransform.TransformPosition(GizmoSpaceSnappedGizmoLocation);
		}
		else
		{
			FSnappingUtils::SnapPointToGrid(SnappedGizmoLocation, SnapGridBase);
		}
	}	

	return SnappedGizmoLocation;
}

void UViewportWorldInteraction::UseLegacyInteractions()
{
	if (DefaultMouseCursorInteractorRefCount == 1)
	{
		this->ReleaseMouseCursorInteractor();
	}
	if (DefaultOptionalViewportClient != nullptr)
	{
		DefaultOptionalViewportClient->ShowWidget(true);
	}
	for (UViewportInteractor* Interactor : Interactors)
	{
		Interactor->Shutdown();
		Interactor->MarkAsGarbage();
	}

	Interactors.Empty();
	Transformables.Empty();
	Colors.Empty();

	OnHoverUpdateEvent.Clear();
	OnPreviewInputActionEvent.Clear();
	OnInputActionEvent.Clear();
	OnKeyInputEvent.Clear();
	OnAxisInputEvent.Clear();
	OnStopDraggingEvent.Clear();

	TransformGizmoActor = nullptr;
	SnapGridActor = nullptr;
	SnapGridMeshComponent = nullptr;
	SnapGridMID = nullptr;
	DraggedInteractable = nullptr;
	if (ViewportTransformer != nullptr)
	{
		ViewportTransformer->Shutdown();
		ViewportTransformer = nullptr;
	}

	AssetContainer = nullptr;

	GizmoType.Reset();

	// Remove the input pre-processor
	if (InputProcessor.IsValid())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(InputProcessor);
		InputProcessor.Reset();
	}
	USelection::SelectionChangedEvent.RemoveAll(this);
}

void UViewportWorldInteraction::UseVWInteractions()
{
	// Add colors
	InitColors();

	// Setup the asset container.
	AssetContainer = LoadAssetContainer();

	if (DefaultMouseCursorInteractorRefCount == 0)
	{
		this->AddMouseCursorInteractor();
	}
	if (DefaultOptionalViewportClient != nullptr)
	{
		DefaultOptionalViewportClient->ShowWidget(false);
	}

	// Start with the default transformer
	SetTransformer(nullptr);

	// Spawn the transform gizmo
	SpawnTransformGizmoIfNeeded();

	const bool bShouldBeVisible = false;
	const bool bPropagateToChildren = true;
	TransformGizmoActor->GetRootComponent()->SetVisibility(bShouldBeVisible, bPropagateToChildren);

	// Create and add the input pre-processor to the slate application.
	if (!InputProcessor.IsValid())
	{
		InputProcessor = MakeShareable(new FViewportInteractionInputProcessor(this));
		FSlateApplication::Get().RegisterInputPreProcessor(InputProcessor);
	}

	// Pretend that actor selection changed, so that our gizmo refreshes right away based on which objects are selected
	GEditor->NoteSelectionChange();
	GEditor->SelectNone(true, true, false);

	CurrentTickNumber = 0;
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
