// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportTransformer.h"
#include "ActorTransformer.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UActorTransformer : public UViewportTransformer
{
	GENERATED_BODY()

public:

	// UViewportTransformer overrides
	UE_API virtual void Init( class UViewportWorldInteraction* InitViewportWorldInteraction ) override;
	UE_API virtual void Shutdown() override;
	virtual bool CanAlignToActors() const override
	{
		return true;
	}
	UE_API virtual void OnStartDragging(class UViewportInteractor* Interactor) override;
	UE_API virtual void OnStopDragging(class UViewportInteractor* Interactor) override;

protected:

	/** Called when selection changes in the level */
	UE_API void OnActorSelectionChanged( UObject* ChangedObject );

};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
