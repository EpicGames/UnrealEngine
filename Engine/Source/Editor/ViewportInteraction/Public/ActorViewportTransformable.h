// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportTransformable.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


/**
 * A transformable actor
 */
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FActorViewportTransformable : public FViewportTransformable
{

public:

	/** Sets up safe defaults */
	FActorViewportTransformable()
		: ActorWeakPtr(),
		  bShouldBeCarried( false )
	{
	}

	// FViewportTransformable overrides
	UE_API virtual const FTransform GetTransform() const override;
	UE_API virtual void ApplyTransform( const FTransform& NewTransform, const bool bSweep ) override;
	UE_API virtual FBox BuildBoundingBox( const FTransform& BoundingBoxToWorld ) const override;
	UE_API virtual bool IsPhysicallySimulated() const override;
	UE_API virtual bool ShouldBeCarried() const override;
	UE_API virtual void SetLinearVelocity( const FVector& NewVelocity ) override;
	UE_API virtual FVector GetLinearVelocity() const override;
	UE_API virtual void UpdateIgnoredActorList( TArray<class AActor*>& IgnoredActors ) override;

	/** The actual actor object */
	TWeakObjectPtr<class AActor> ActorWeakPtr;
	/** whether this actor should be 'carried' (moved and rotated) when dragged, if possible, instead of only translated  */
	bool bShouldBeCarried;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
