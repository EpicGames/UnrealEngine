// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ViewportInteractor.h"
#include "MouseCursorInteractor.generated.h"

#define UE_API VIEWPORTINTERACTION_API

/**
 * Viewport interactor for a mouse cursor
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UMouseCursorInteractor : public UViewportInteractor
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	GENERATED_BODY()

public:

	/** Default constructor. */
	UE_API UMouseCursorInteractor();

	/** Initialize default values */
	UE_API void Init();

	// ViewportInteractor overrides
	UE_API virtual void PollInput() override;
	UE_API virtual bool IsModifierPressed() const override;

protected:

	UE_API virtual bool AllowLaserSmoothing() const;

private:

	/** Whether the control key was pressed the last time input was polled */
	bool bIsControlKeyPressed;

};

#undef UE_API
