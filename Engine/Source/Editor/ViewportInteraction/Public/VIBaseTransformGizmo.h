// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "UnrealWidgetFwd.h"
#include "ViewportInteractionTypes.h"
#include "VIBaseTransformGizmo.generated.h"


#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UMaterialInterface;

UENUM()
enum class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") EGizmoHandleTypes : uint8
{
	All = 0,
	Translate = 1,
	Rotate = 2,
	Scale = 3
};

/**
* Displays measurements along the bounds of selected objects
*/
USTRUCT()
struct UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FTransformGizmoMeasurement
{
	GENERATED_BODY()

	/** The text that displays the actual measurement and units */
	UPROPERTY()
	TObjectPtr<class UTextRenderComponent> MeasurementText = nullptr;
};


/**
 * Base class for transform gizmo
 */
UCLASS(MinimalAPI, Abstract, NotPlaceable)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") ABaseTransformGizmo : public AActor
{
	GENERATED_BODY()

public:
	
	/** Default constructor that sets up CDO properties */
	UE_API ABaseTransformGizmo();

	//~ Begin AActor interface
	virtual bool IsEditorOnly() const final
	{
		return true;
	}
	//~ End AActor interface

	/** Call this when new objects become selected.  This triggers an animation transition. */
	UE_API void OnNewObjectsSelected();

	/** Called by the world interaction system when one of our components is dragged by the user to find out
	    what type of interaction to do.  If null is passed in then we'll treat it as dragging the whole object 
		(rather than a specific axis/handle) */
	UE_API class UViewportDragOperationComponent* GetInteractionType( UActorComponent* DraggedComponent, TOptional<FTransformGizmoHandlePlacement>& OutHandlePlacement );

	/** Updates the animation with the current time and selected time */
	UE_API float GetAnimationAlpha();

	/** Sets the owner */
	UE_API void SetOwnerWorldInteraction( class UViewportWorldInteraction* InWorldInteraction );

	/** Gets the owner */
	UE_API class UViewportWorldInteraction* GetOwnerWorldInteraction() const;

	/** Called by the world interaction system after we've been spawned into the world, to allow
	    us to create components and set everything up nicely for the selected objects that we'll be
		used to manipulate */
	virtual void UpdateGizmo(const EGizmoHandleTypes InGizmoType, const ECoordSystem InGizmoCoordinateSpace, const FTransform& InLocalToWorld, const FBox& InLocalBounds, const FVector& InViewLocation, const float InScaleMultiplier, bool bInAllHandlesVisible,
		const bool bInAllowRotationAndScaleHandles, class UActorComponent* DraggingHandle, const TArray<UActorComponent*>& InHoveringOverHandles, const float InGizmoHoverScale, const float InGizmoHoverAnimationDuration) {}


	/** Gets the current gizmo handle type */
	UE_API EGizmoHandleTypes GetGizmoType() const;

protected:

	/** Static: Given a bounding box and information about which edge to query, returns the vertex positions of that edge */
	static UE_API void GetBoundingBoxEdge( const FBox& Box, const int32 AxisIndex, const int32 EdgeIndex, FVector& OutVertex0, FVector& OutVertex1 );

	/** Updates the visibility of all the handles */
	UE_API void UpdateHandleVisibility(const EGizmoHandleTypes InGizmoType, const ECoordSystem InGizmoCoordinateSpace, bool bInAllHandlesVisible, UActorComponent* DraggingHandle);

	/** Gets if the gizmo shows measurement texts */
	UE_API bool GetShowMeasurementText() const;

	/** Real time that the gizmo was last attached to a selected set of objects.  This is used for animation transitions */
	FTimespan SelectedAtTime;
	
	/** Scene component root of this actor */
	UPROPERTY()
	TObjectPtr<USceneComponent> SceneComponent;

	/** All gizmo components */
	UPROPERTY()
	TArray< TObjectPtr<class UGizmoHandleGroup> > AllHandleGroups;

	/** Owning object */
	UPROPERTY()
	TObjectPtr<class UViewportWorldInteraction> WorldInteraction;
	
	/** Current gizmo type */
	EGizmoHandleTypes GizmoType;
};	


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
