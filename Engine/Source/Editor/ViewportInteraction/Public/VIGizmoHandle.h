// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UnrealWidgetFwd.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "ViewportInteractionTypes.h"
#include "VIGizmoHandle.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UMaterialInterface;
class UStaticMesh;
enum class EGizmoHandleTypes : uint8;
class UActorComponent;

USTRUCT()
struct UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FGizmoHandle
{
	GENERATED_BODY()

	/** Static mesh for this handle */
	class UGizmoHandleMeshComponent* HandleMesh;

	/** Scalar that will advance toward 1.0 over time as we hover over the gizmo handle */
	float HoverAlpha;
};


/**
 * Base class for gizmo handles
 */
UCLASS(MinimalAPI,  ABSTRACT )
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UGizmoHandleGroup : public USceneComponent
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UE_API UGizmoHandleGroup();
	
	/** Given the unique index, makes a handle */
	UE_API FTransformGizmoHandlePlacement MakeHandlePlacementForIndex( const int32 HandleIndex ) const;

	/** Makes a unique index for a handle */
	UE_API int32 MakeHandleIndex( const FTransformGizmoHandlePlacement HandlePlacement ) const;

	/** Makes up a name string for a handle */
	UE_API FString MakeHandleName( const FTransformGizmoHandlePlacement HandlePlacement ) const;

	/** Static: Given an axis (0-2) and a facing direction, returns the vector normal for that axis */
	static UE_API FVector GetAxisVector( const int32 AxisIndex, const ETransformGizmoHandleDirection HandleDirection );

	/** Updates the Gizmo handles, needs to be implemented by derived classes */
	UE_API virtual void UpdateGizmoHandleGroup( const FTransform& LocalToWorld, const FBox& LocalBounds, const FVector ViewLocation, const bool bAllHandlesVisible, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, 
		float AnimationAlpha, float GizmoScale, const float GizmoHoverScale, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup );

	/** Default setting the visibility and collision for all the handles in this group */
	UE_API void UpdateVisibilityAndCollision(const EGizmoHandleTypes GizmoType, const ECoordSystem GizmoCoordinateSpace, const bool bAllHandlesVisible, const bool bAllowRotationAndScaleHandles, UActorComponent* DraggingHandle);

	UE_API class UViewportDragOperationComponent* GetDragOperationComponent();

	/** Finds the index of DraggedMesh in HandleMeshes */
	UE_API virtual int32 GetDraggedHandleIndex( class UStaticMeshComponent* DraggedMesh );

	/** Sets the Gizmo material */
	UE_API void SetGizmoMaterial( UMaterialInterface* Material );
	
	/** Sets the translucent Gizmo material */
	UE_API void SetTranslucentGizmoMaterial( UMaterialInterface* Material );

	/** Gets all the handles */
	UE_API TArray< FGizmoHandle >& GetHandles();

	/** Gets the GizmoType for this Gizmo handle */
	UE_API virtual EGizmoHandleTypes GetHandleType() const;

	/** Returns true if this type of handle is allowed with world space gizmos */
	virtual bool SupportsWorldCoordinateSpace() const
	{
		return true;
	}

	/** Sets if this handlegroup will be visible with the universal gizmo */
	UE_API void SetShowOnUniversalGizmo( const bool bShowHandleUniversal );
	
	/** Gets if this handlegroup will be visible with the universal gizmo */
	UE_API bool GetShowOnUniversalGizmo() const;

	/** Sets the owning transform gizmo for this handle group*/
	UE_API void SetOwningTransformGizmo(class ABaseTransformGizmo* TransformGizmo); 

protected:
	/** Updates the colors of the dynamic material instances for the handle passed using its axis index */	
	UE_API void UpdateHandleColor( const int32 AxisIndex, FGizmoHandle& Handle, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles );

	/** Helper function to create gizmo handle meshes */
	UE_API class UGizmoHandleMeshComponent* CreateMeshHandle( class UStaticMesh* HandleMesh, const FString& ComponentName );

	/** Creates handle meshcomponent and adds it to the Handles list */
	UE_API class UGizmoHandleMeshComponent* CreateAndAddMeshHandle( class UStaticMesh* HandleMesh, const FString& ComponentName, const FTransformGizmoHandlePlacement& HandlePlacement );

	/** Adds the HandleMeshComponent to the Handles list */
	UE_API void AddMeshToHandles( class UGizmoHandleMeshComponent* HandleMeshComponent, const FTransformGizmoHandlePlacement& HandlePlacement );

	/** Gets the handleplacement axes */
	UE_API FTransformGizmoHandlePlacement GetHandlePlacement( const int32 X, const int32 Y, const int32 Z ) const;

	/** Gizmo material (opaque) */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> GizmoMaterial;

	/** Gizmo material (translucent) */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> TranslucentGizmoMaterial;
	
	/** All the StaticMeshes for this handle type */
	UPROPERTY()
	TArray< FGizmoHandle > Handles;

	/** The actor transform gizmo owning this handlegroup */
	UPROPERTY()
	TObjectPtr<class ABaseTransformGizmo> OwningTransformGizmoActor;

	UPROPERTY()
	TObjectPtr<class UViewportDragOperationComponent> DragOperationComponent;

private:

	/** Updates the hover animation for the HoveringOverHandles */
	UE_API void UpdateHoverAnimation( class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles, const float GizmoHoverAnimationDuration, bool& bOutIsHoveringOrDraggingThisHandleGroup );

	/** If this handlegroup will be visible with the universal gizmo */
	bool bShowOnUniversalGizmo;
};


/**
 * Base class for gizmo handles on axis
 */
UCLASS(MinimalAPI, ABSTRACT)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UAxisGizmoHandleGroup : public UGizmoHandleGroup
{
	GENERATED_BODY()

protected:

	/** Creates mesh for every axis */
	UE_API void CreateHandles(UStaticMesh* HandleMesh, const FString& HandleComponentName);
	
	/** 
	 * Calculates the transforms of meshes of this handlegroup 
	 * @param HandleToCenter - The offset from the center of the gizmo in roomspace
	 */
	UE_API void UpdateHandlesRelativeTransformOnAxis( const FTransform& HandleToCenter, const float AnimationAlpha, const float GizmoScale, const float GizmoHoverScale, 
		const FVector& ViewLocation, class UActorComponent* DraggingHandle, const TArray< UActorComponent* >& HoveringOverHandles );
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
