// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/StaticMeshComponent.h"
#include "VIGizmoHandleMeshComponent.generated.h"

UCLASS(hidecategories = Object)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UGizmoHandleMeshComponent : public UStaticMeshComponent
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UGizmoHandleMeshComponent();

	//~ Begin UPrimitiveComponent Interface
	virtual class FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface
};