// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VISettings.generated.h"

/**
* Implements the settings for Viewport Interaction and VR Mode. Note that this is primarily a base class for anything that needs to be accessed in the VI module.
*/
UCLASS(MinimalAPI, config = EditorSettings)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UVISettings : public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UVISettings()
		: Super() 
	{
	};

	/** Whether the world should scale relative to your tracking space floor instead of the center of your hand locations */
	UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.")
	UPROPERTY(config)
	uint32 bScaleWorldFromFloor : 1;

	/** Whether to compute a new center point for scaling relative from by looking at how far either controller moved relative to the last frame */
	UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.")
	UPROPERTY(config)
	uint32 bScaleWorldWithDynamicPivot : 1;

	/** When enabled, you can freely rotate and scale the world with two hands at the same time.  Otherwise, we'll detect whether to rotate or scale depending on how much of either gesture you initially perform. */
	UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.")
	UPROPERTY(config)
	uint32 bAllowSimultaneousWorldScalingAndRotation : 1;
};

