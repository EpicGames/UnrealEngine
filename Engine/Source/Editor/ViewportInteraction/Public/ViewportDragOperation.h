// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/Box.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealWidgetFwd.h"
#include "ViewportInteractionTypes.h"

#include "ViewportDragOperation.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


/**
 * Base class for interactable drag calculations
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UViewportDragOperation : public UObject
{
	GENERATED_BODY()

public:

	/** 
	 * Execute dragging 
	 *
	 * @param UViewportInteractor - The interactor causing the dragging
	 * @param IViewportInteractableInterface - The interactable owning this drag operation
	 */
	virtual void ExecuteDrag(class UViewportInteractor* Interactor, class IViewportInteractableInterface* Interactable){};

	/** 
	 * Execute dragging 
	 *
	 * @param UViewportInteractor - The interactor causing the dragging
	 * @param IViewportInteractableInterface - The interactable owning this drag operation
	 */
	virtual void ExecuteDrag(struct FDraggingTransformableData& DraggingData){};

	/** @todo ViewportInteraction: Temporary way to check if a drag operation needs to constrain the drag delta on a plane of a gizmo axis. */
	bool bPlaneConstraint;
};


/**
 * Container component for UViewportDragOperation that can be used by objects in the world that are draggable and implement the ViewportInteractableInterface
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UViewportDragOperationComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	
	/** Default constructor. */
	UE_API UViewportDragOperationComponent();

	/** Get the actual dragging operation */
	UE_API UViewportDragOperation* GetDragOperation();

	/** Sets the drag operation class that will be used next time starting dragging */
	UE_API void SetDragOperationClass( const TSubclassOf<UViewportDragOperation> InDragOperation );

	/** Starts new dragging operation */
	UE_API void StartDragOperation();

	/* Destroys the dragoperation */
	UE_API void ClearDragOperation();

	/** If this operation is currently active. */
	UE_API bool IsDragging() const;

private:

	/** The actual dragging implementation */
	UPROPERTY()
	TObjectPtr<UViewportDragOperation> DragOperation;

	/** The next class that will be used as drag operation */
	UPROPERTY()
	TSubclassOf<UViewportDragOperation> DragOperationSubclass;
};

/**
 * Data structure that holds all arguments that can be used while dragging a transformable.
 */
USTRUCT()
struct UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FDraggingTransformableData
{
	GENERATED_BODY()

	FDraggingTransformableData() :
		Interactor(nullptr),
		WorldInteraction(nullptr),
		PassDraggedTo(FVector::ZeroVector),
		OptionalHandlePlacement(),
		DragDelta(FVector::ZeroVector),
		ConstrainedDragDelta(FVector::ZeroVector),
		OtherHandDragDelta(FVector::ZeroVector),
		DraggedTo(FVector::ZeroVector),
		OtherHandDraggedTo(FVector::ZeroVector),
		DragDeltaFromStart(FVector::ZeroVector),
		OtherHandDragDeltaFromStart(FVector::ZeroVector),
		LaserPointerStart(FVector::ZeroVector),
		LaserPointerDirection(FVector::ZeroVector),
		GizmoStartTransform(FTransform::Identity),
		GizmoLastTransform(FTransform::Identity),
		OutGizmoUnsnappedTargetTransform(FTransform::Identity),
		GizmoStartLocalBounds(ForceInitToZero),
		GizmoCoordinateSpace(ECoordSystem::COORD_World),
		bOutMovedTransformGizmo(false),
		bOutShouldApplyVelocitiesFromDrag(false),
		OutUnsnappedDraggedTo(FVector::ZeroVector),
		bOutTranslated(false),
		bOutRotated(false),
		bOutScaled(false),
		bAllowSnap(true)
	{}

	class UViewportInteractor* Interactor;
	class UViewportWorldInteraction* WorldInteraction;

	FVector PassDraggedTo;
	TOptional<FTransformGizmoHandlePlacement> OptionalHandlePlacement;
	FVector DragDelta;
	FVector ConstrainedDragDelta;
	FVector OtherHandDragDelta;
	FVector DraggedTo;
	FVector OtherHandDraggedTo;
	FVector DragDeltaFromStart;
	FVector OtherHandDragDeltaFromStart;
	FVector LaserPointerStart;
	FVector LaserPointerDirection;
	FTransform GizmoStartTransform;
	FTransform GizmoLastTransform;
	FTransform OutGizmoUnsnappedTargetTransform;
	FBox GizmoStartLocalBounds;
	ECoordSystem GizmoCoordinateSpace;

	bool bOutMovedTransformGizmo;
	bool bOutShouldApplyVelocitiesFromDrag;
	FVector OutUnsnappedDraggedTo;
	bool bOutTranslated;
	bool bOutRotated;
	bool bOutScaled;
	bool bAllowSnap;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
