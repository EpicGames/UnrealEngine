// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ViewportInteractableInterface.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UActorComponent;
class UObject;
class UViewportInteractor;
struct FHitResult;

UINTERFACE( MinimalAPI )
class
#if CPP
	UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.")
#endif
	UViewportInteractableInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * Interface for custom objects that can be interacted with by a interactor
 */
class
#if CPP
	UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.")
#endif
	IViewportInteractableInterface
{
	GENERATED_IINTERFACE_BODY()

public:

	/** Called when an interactor presses this object */
	virtual void OnPressed( UViewportInteractor* Interactor, const FHitResult& InHitResult, bool& bOutResultedInDrag ) PURE_VIRTUAL( IViewportInteractableInterface::OnPressed, );
	
	/** Called when an interactor hover over this object */
	virtual void OnHover( UViewportInteractor* Interactor ) PURE_VIRTUAL( IViewportInteractableInterface::OnHover, );
	
	/** Called when an interactor starts hovering over this object */
	virtual void OnHoverEnter( UViewportInteractor* Interactor, const FHitResult& InHitResult ) PURE_VIRTUAL( IViewportInteractableInterface::OnHoverEnter, );
	
	/** Called when an interactor leave hovering over this object */
	virtual void OnHoverLeave( UViewportInteractor* Interactor, const UActorComponent* NewComponent ) PURE_VIRTUAL( IViewportInteractableInterface::OnHoverLeave, );
	
	/** Called when an interactor stops dragging this object */
	virtual void OnDragRelease( UViewportInteractor* Interactor ) PURE_VIRTUAL( IViewportInteractableInterface::OnDragRelease, );

	/** Get dragging operation */
	virtual class UViewportDragOperationComponent* GetDragOperationComponent() PURE_VIRTUAL( IViewportInteractableInterface::GetDragOperationComponent, return nullptr; );

	/** Whether this interactable can be selected. */
	virtual bool CanBeSelected() PURE_VIRTUAL( IViewportInteractableInterface::CanBeSelected, return false; );
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
