// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "ViewportInteractionAssetContainer.generated.h"

// Forward declarations
class UMaterialInterface;
class USoundBase;
class UStaticMesh;

/**
 * Asset container for viewport interaction.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UViewportInteractionAssetContainer : public UDataAsset
{
	GENERATED_BODY()

public:

	//
	// Sound
	//

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> GizmoHandleSelectedSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> GizmoHandleDropSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> SelectionChangeSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> SelectionDropSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> SelectionStartDragSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> GridSnapSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> ActorSnapSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> UndoSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> RedoSound;

	//
	// Meshes
	//

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> GridMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> TranslationHandleMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> UniformScaleHandleMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> ScaleHandleMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> PlaneTranslationHandleMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> RotationHandleMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> RotationHandleSelectedMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> StartRotationIndicatorMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> CurrentRotationIndicatorMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> FreeRotationHandleMesh;

	//
	// Materials
	//

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> GridMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> TransformGizmoMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> TranslucentTransformGizmoMaterial;
};
