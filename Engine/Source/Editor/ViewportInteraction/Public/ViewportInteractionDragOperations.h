// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ViewportDragOperation.h"	
#include "ViewportInteractionDragOperations.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


/**
 * Gizmo translation on one axis.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UTranslationDragOperation : public UViewportDragOperation
{
	GENERATED_BODY()

public:
	UE_API virtual void ExecuteDrag(struct FDraggingTransformableData& DraggingData) override;
};

/**
 * Gizmo translation on two axes.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UPlaneTranslationDragOperation : public UViewportDragOperation
{
	GENERATED_BODY()

public:
	UE_API UPlaneTranslationDragOperation();
	UE_API virtual void ExecuteDrag(struct FDraggingTransformableData& DraggingData) override;
};

/**
 * Rotation around one axis based on input angle.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") URotateOnAngleDragOperation : public UViewportDragOperation
{
	GENERATED_BODY()

public:
	UE_API URotateOnAngleDragOperation();
	
	UE_API virtual void ExecuteDrag(struct FDraggingTransformableData& DraggingData) override;

	/** When RotateOnAngle we intersect on a plane to rotate the transform gizmo. This is the local point from the transform gizmo location of that intersect */
	UE_API FVector GetLocalIntersectPointOnRotationGizmo() const;

private:

	/** Starting angle when rotating an object with  ETransformGizmoInteractionType::RotateOnAngle */
	TOptional<float> StartDragAngleOnRotation;

	/** The direction of where the rotation handle is facing when starting rotation */
	TOptional<FVector> DraggingRotationHandleDirection;

	/** Where the laser intersected on the gizmo rotation aligned plane. */
	FVector LocalIntersectPointOnRotationGizmo;
};

/**
 * Scale on one axis.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UScaleDragOperation : public UViewportDragOperation
{
	GENERATED_BODY()

public:
	UE_API virtual void ExecuteDrag(struct FDraggingTransformableData& DraggingData) override;
};

/**
 * Scale on all axes.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UUniformScaleDragOperation : public UViewportDragOperation
{
	GENERATED_BODY()

public:
	UE_API virtual void ExecuteDrag(struct FDraggingTransformableData& DraggingData) override;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
