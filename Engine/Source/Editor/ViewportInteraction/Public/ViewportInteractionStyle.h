// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformMath.h"
#include "Styling/SlateStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ISlateStyle;
struct FSlateBrush;

/**  */
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FViewportInteractionStyle
{
public:

	static void Initialize();

	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for the UMG Style */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return ViewportInteractionStyleInstance->GetBrush(PropertyName, Specifier);
	}

private:

	static TSharedRef< class FSlateStyleSet > Create();

private:

	static TSharedPtr< class FSlateStyleSet > ViewportInteractionStyleInstance;

	
};