// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"
#include "ViewportInteractionTypes.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


/**
 * Represents a generic action
*/
USTRUCT( BlueprintType )
struct UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FViewportActionKeyInput
{
	GENERATED_BODY()
	
	FViewportActionKeyInput()
		: ActionType( NAME_None )
		, Event( IE_Pressed )
		, bIsInputCaptured( false )
		, bIsAxis( false )
	{}

	FViewportActionKeyInput( const FName& InActionType )
		: ActionType( InActionType )
		, Event( IE_Pressed )
		, bIsInputCaptured( false )
		, bIsAxis( false )
	{}

	static FViewportActionKeyInput Axis(const FName& InActionType)
	{
		FViewportActionKeyInput Input(InActionType);
		Input.bIsAxis = true;
		return Input;
	}

	/** The name of this action */
	UPROPERTY( BlueprintReadOnly, Category = Action )
	FName ActionType;

	/** Input event */
	UPROPERTY(BlueprintReadOnly, Category = Action)
	TEnumAsByte<EInputEvent> Event;

	/** True if this action owned by an interactor is "captured" for each possible action type, meaning that only the active captor should 
	handle input events until it is no longer captured.  It's the captors responsibility to set this using OnVRAction(), or clear it when finished with capturing. */
	UPROPERTY()
	bool bIsInputCaptured;

	UPROPERTY()
	bool bIsAxis;

	bool operator==(const FViewportActionKeyInput& Other) const
	{
		return ActionType == Other.ActionType &&
			Event == Other.Event &&
			bIsInputCaptured == Other.bIsInputCaptured &&
			bIsAxis == Other.bIsAxis;
	}

	friend uint32 GetTypeHash(const FViewportActionKeyInput& Input)
	{
		uint32 Hash = GetTypeHash(Input.ActionType);
		Hash = HashCombine(Hash, GetTypeHash(Input.Event));
		Hash = HashCombine(Hash, GetTypeHash(Input.bIsInputCaptured));
		Hash = HashCombine(Hash, GetTypeHash(Input.bIsAxis));
		return Hash;
	}
};


/** Methods of dragging objects around in VR */
UENUM()
enum class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") EViewportInteractionDraggingMode : uint8
{
	/** Not dragging right now with this hand */
	Nothing,

	/** Dragging transformables (e.g. actors, components, geometry elements) around using the transform gizmo */
	TransformablesWithGizmo,

	/** Transformables locked to the impact point under the laser */
	TransformablesAtLaserImpact,

	/** We're grabbing an object (or the world) that was already grabbed by the other hand */
	AssistingDrag,

	/** Freely moving, rotating and scaling transformables with one or two hands */
	TransformablesFreely,

	/** Moving the world itself around (actually, moving the camera in such a way that it feels like you're moving the world) */
	World,

	/** Moving a custom interactable */
	Interactable,

	/** Dragging a material */
	Material
};

/* Directions that a transform handle can face along any given axis */
enum class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") ETransformGizmoHandleDirection
{
	Negative,
	Center,
	Positive,
};


/** Placement of a handle in pivot space */
USTRUCT()
struct UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FTransformGizmoHandlePlacement
{
	GENERATED_BODY()

	/* Handle direction in X, Y and Z */
	ETransformGizmoHandleDirection Axes[ 3 ];


	/** Finds the center handle count and facing axis index.  The center handle count is simply the number
	of axes where the handle would be centered on the bounds along that axis.  The facing axis index is
	the index (0-2) of the axis where the handle would be facing, or INDEX_NONE for corners or edges.
	The center axis index is valid for edges, and defines the axis perpendicular to that edge direction,
	or INDEX_NONE if it's not an edge */
	UE_API void GetCenterHandleCountAndFacingAxisIndex( int32& OutCenterHandleCount, int32& OutFacingAxisIndex, int32& OutCenterAxisIndex ) const;
};


enum class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") ELockedWorldDragMode
{
	Unlocked,
	OnlyRotating,
	OnlyScaling,
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
