// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"

#define UE_API VIEWPORTINTERACTION_API

namespace ViewportInteractionUtils
{
	/************************************************************************/
	/* 1 Euro filter smoothing algorithm									*/
	/* http://cristal.univ-lille.fr/~casiez/1euro/							*/
	/************************************************************************/
	class FOneEuroFilter
	{
	private:

		class FLowpassFilter
		{
		public:

			/** Default constructor */
			FLowpassFilter();

			/** Calculate */
			FVector Filter(const FVector& InValue, const FVector& InAlpha);

			/** If the filter was not executed yet */
			bool IsFirstTime() const;

			/** Get the previous filtered value */
			FVector GetPrevious() const;

		private:

			/** The previous filtered value */
			FVector Previous;

			/** If this is the first time doing a filter */
			bool bFirstTime;
		};

	public:

		/** Default constructor */
		UE_API FOneEuroFilter();

		UE_API FOneEuroFilter(const double InMinCutoff, const double InCutoffSlope, const double InDeltaCutoff);

		/** Smooth vector */
		UE_API FVector Filter(const FVector& InRaw, const double InDeltaTime);

		/** Set the minimum cutoff */
		UE_API void SetMinCutoff(const double InMinCutoff);

		/** Set the cutoff slope */
		UE_API void SetCutoffSlope(const double InCutoffSlope);

		/** Set the delta slope */
		UE_API void SetDeltaCutoff(const double InDeltaCutoff);

	private:

		UE_API const FVector CalculateCutoff(const FVector& InValue);
		UE_API const FVector CalculateAlpha(const FVector& InCutoff, const double InDeltaTime) const;
		UE_API const double CalculateAlpha(const double InCutoff, const double InDeltaTime) const;

		double MinCutoff;
		double CutoffSlope;
		double DeltaCutoff;
		FLowpassFilter RawFilter;
		FLowpassFilter DeltaFilter;
	};
}

#undef UE_API
