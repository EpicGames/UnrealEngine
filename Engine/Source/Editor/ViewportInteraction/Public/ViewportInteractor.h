// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Engine/EngineTypes.h"
#include "ViewportInteractionTypes.h"
#include "ViewportInteractorData.h"
#include "ViewportInteractionUtils.h"
#include "ViewportInteractor.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class AActor;

/** Filter mode for GetHitResultFromLaserPointer*/
UENUM(BlueprintType)
enum class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") EHitResultGizmoFilterMode : uint8
{
	All,
	NoGizmos,
	GizmosOnly
};

/**
 * Represents the interactor in the world
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UViewportInteractor : public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UE_API UViewportInteractor();

	/** Gets the private data for this interactor */
	UE_API struct FViewportInteractorData& GetInteractorData();

	/** Gets the private data for this interactor (const) */
	UE_API const struct FViewportInteractorData& GetInteractorData() const;

	/** Sets the world interaction */
	UE_API void SetWorldInteraction( class UViewportWorldInteraction* InWorldInteraction );

	/** Gets the world interaction */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API UViewportWorldInteraction* GetWorldInteraction() const;

	/** Sets the other interactor */
	UE_API void SetOtherInteractor( UViewportInteractor* InOtherInteractor ); //@todo ViewportInteraction: This should not be public to other modules and should only be called from the world interaction.

	/** Removes the other interactor reference for this interactor */
	UE_API void RemoveOtherInteractor();

	/** Gets the paired interactor assigned by the world interaction, can return null when there is no other interactor */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API UViewportInteractor* GetOtherInteractor() const;

	/** Whenever the ViewportWorldInteraction is shut down, the interacts will shut down as well */
	UFUNCTION(BlueprintNativeEvent, CallInEditor, Category = "Interactor")
	UE_API void Shutdown();

	/** Update for this interactor called by the ViewportWorldInteraction */
	UFUNCTION(BlueprintNativeEvent, CallInEditor, Category = "Interactor")
	UE_API void Tick( const float DeltaTime );

	/** Gets the last component hovered on by the interactor laser. */
	UE_API virtual class UActorComponent* GetLastHoverComponent();

	/** Adds a new action to the KeyToActionMap */
	UE_API void AddKeyAction( const FKey& Key, const FViewportActionKeyInput& Action );

	/** Removes an action from the KeyToActionMap */
	UE_API void RemoveKeyAction( const FKey& Key );

	/** Base classes need to implement getting the input for the input devices for that interactor */
	virtual void PollInput() {}

	/**
	 * Handles key input and translates it actions 
	 *  C++ Child classes are expected to to call there super versions of this.
	 *  BP will have there "Receive" versions called from within these function and do not have to call
	 *  there parents.  They simply need to return if they handled the input or not.
	 */
	UE_API bool HandleInputKey( class FEditorViewportClient& ViewportClient, const FKey Key, const EInputEvent Event );

	/**
	 * Handles axis input and translates it to actions
	 * C++ Child classes are expected to to call there super versions of this.
	 *  BP will have there "Receive" versions called from within these function and do not have to call
	 *  there parents.They simply need to return if they handled the input or not.
	 */
	UE_API bool HandleInputAxis( class FEditorViewportClient& ViewportClient, const FKey Key, const float Delta, const float DeltaTime );

	/** @return	Returns true if this interactor's designated 'modifier button' is currently held down.  Some interactors may not support this */
	virtual bool IsModifierPressed() const
	{
		return false;
	}

	/** Gets the world transform of this interactor */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API FTransform GetTransform() const;

	/** Gets the hand transform of the interactor, in the local tracking space */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API FTransform GetRoomSpaceTransform() const;

	/** Gets the last world transform of this interactor */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API FTransform GetLastTransform() const;

	/** Gets the last hand transform of the interactor, in the local tracking space */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API FTransform GetLastRoomSpaceTransform() const;

	/** Gets the current interactor data dragging mode */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API EViewportInteractionDraggingMode GetDraggingMode() const;

	/** Gets the interactor data previous dragging mode */
	UE_API EViewportInteractionDraggingMode GetLastDraggingMode() const;

	/** Gets the interactor data drag velocity */
	UE_API FVector GetDragTranslationVelocity() const;

	/** Sets the hover location */
	UE_API void SetHoverLocation(const FVector& InHoverLocation);

	/**
	 * Gets the start and end point of the laser pointer for the specified hand
	 *
	 * @param LasertPointerStart	(Out) The start location of the laser pointer in world space
	 * @param LasertPointerEnd		(Out) The end location of the laser pointer in world space
	 * @param bEvenIfBlocked		If true, returns a laser pointer even if the hand has UI in front of it (defaults to false)
	 * @param LaserLengthOverride	If zero the default laser length (VREdMode::GetLaserLength) is used
	 *
	 * @return	True if we have motion controller data for this hand and could return a valid result
	 */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API bool GetLaserPointer( FVector& LaserPointerStart, FVector& LaserPointerEnd, const bool bEvenIfBlocked = false, const float LaserLengthOverride = 0.0f );

	/**
	 * Gets a sphere on this interactor that can be used to interact with objects in close proximity
	 *
	 * @param	OutGrabberSphere	The sphere in world space
	 * @param	bEvenIfBlocked	When set to true, a valid sphere may be returned even if there is UI attached in front of this interactor
	 *
	 * @return	True if the sphere is available and valid, or false is the interactor was busy or we could not determine a valid position for the interactor
	 */
	UE_API bool GetGrabberSphere( FSphere& OutGrabberSphere, const bool bEvenIfBlocked = false );

	/** Gets the maximum length of a laser pointer */
	UE_API float GetLaserPointerMaxLength() const;

	/** Triggers a force feedback effect on device if possible */
	virtual void PlayHapticEffect( const float Strength ) {};

	/**
	 * Traces along the laser pointer vector and returns what it first hits in the world
	 *
	 * @param OptionalListOfIgnoredActors Actors to exclude from hit testing
	 * @param bIgnoreGizmos True if no gizmo results should be returned, otherwise they are preferred (x-ray)
	 * @param bEvenIfUIIsInFront If true, ignores any UI that might be blocking the ray
	 * @param LaserLengthOverride If zero the default laser length (VREdMode::GetLaserLength) is used
	 *
	 * @return What the laster pointer hit
	 * @deprecated bool bIgnoreGizmos replaced with EHitResultGizmoFilterMode GizmoFilterMode
	 */
	UE_DEPRECATED(4.23, "bool bIgnoreGizmos replaced with EHitResultGizmoFilterMode GizmoFilterMode")
	virtual FHitResult GetHitResultFromLaserPointer(TArray<AActor*>* OptionalListOfIgnoredActors, const bool bIgnoreGizmos = false, TArray<UClass*>* ObjectsInFrontOfGizmo = nullptr, const bool bEvenIfBlocked = false, const float LaserLengthOverride = 0.0f) 
	{
		return GetHitResultFromLaserPointer(OptionalListOfIgnoredActors, bIgnoreGizmos ? EHitResultGizmoFilterMode::All : EHitResultGizmoFilterMode::NoGizmos, ObjectsInFrontOfGizmo, bEvenIfBlocked, LaserLengthOverride);
	}

	/**
	 * Traces along the laser pointer vector and returns what it first hits in the world
	 *
	 * @param OptionalListOfIgnoredActors Actors to exclude from hit testing
	 * @param GizmoFilterMode filters the types of gizmos for the hit test
	 * @param bEvenIfUIIsInFront If true, ignores any UI that might be blocking the ray
	 * @param LaserLengthOverride If zero the default laser length (VREdMode::GetLaserLength) is used
	 *
	 * @return What the laster pointer hit
	 */
	UE_API virtual FHitResult GetHitResultFromLaserPointer( TArray<AActor*>* OptionalListOfIgnoredActors = nullptr, const EHitResultGizmoFilterMode GizmoFilterMode = EHitResultGizmoFilterMode::All, TArray<UClass*>* ObjectsInFrontOfGizmo = nullptr, const bool bEvenIfBlocked = false, const float LaserLengthOverride = 0.0f );

	/** Reset the values before checking the hover actions */
	UE_API virtual void ResetHoverState();

	/** Needs to be implemented by the base to calculate drag ray length and the velocity for the ray */
	virtual void CalculateDragRay( double& InOutDragRayLength, double& InOutDragRayVelocity ) {};

	UE_DEPRECATED(5.2, "CalculateDragRay() now uses double-precision arguments.")
	virtual void CalculateDragRay( float& InOutDragRayLength, float& InOutDragRayVelocity ) {};

	/**
	 * Creates a hand transform and forward vector for a laser pointer for a given hand
	 *
	 * @param OutHandTransform	The created hand transform
	 * @param OutForwardVector	The forward vector of the hand
	 *
	 * @return	True if we have motion controller data for this hand and could return a valid result
	 */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API virtual bool GetTransformAndForwardVector( FTransform& OutHandTransform, FVector& OutForwardVector ) const;

	/** Called by StartDragging in world interaction to give the interactor a chance of acting upon starting a drag operation */
	UE_API virtual void OnStartDragging( const FVector& HitLocation, const bool bIsPlacingNewObjects );

	/** Gets the interactor laser hover location */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API FVector GetHoverLocation();

	/** If the interactor laser is currently hovering */
	UE_API bool IsHovering() const;

	/** If the interactor laser is currently hovering over a gizmo handle */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API bool IsHoveringOverGizmo() const;

	/** Sets the current dragging mode for this interactor */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API void SetDraggingMode( const EViewportInteractionDraggingMode NewDraggingMode );

	/** To be overridden by base class. Called by GetLaserPointer to give the derived interactor a chance to disable the laser. By default it is not blocked */
	virtual bool GetIsLaserBlocked() const { return false; }

	/** Gets a certain action by iterating through the map looking for the same ActionType */
	// @todo ViewportInteractor: This should be changed to return a const pointer, but we need to fix up some dragging code in WorldInteraction first
	UE_API FViewportActionKeyInput* GetActionWithName( const FName InActionName );

	/** Gets the drag haptic feedback strength console variable */
	UE_API float GetDragHapticFeedbackStrength() const;

	/** If this interactor is hovering over a type that has priority from GetHitResultFromLaserPointer */
	UE_API bool IsHoveringOverPriorityType() const;

	/** Returns true if currently hovering over selected actor */
	UE_API bool IsHoveringOverSelectedActor() const;

	/** Reset the stored laser end location at the end of tick */
	UE_API void ResetLaserEnd();

	/** Sets the current gizmo filter mode used for Interaction*/
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API void SetHitResultGizmoFilterMode(EHitResultGizmoFilterMode newFilter);

	/** Gets current gizmo filter mode used for Interaction*/
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API EHitResultGizmoFilterMode GetHitResultGizmoFilterMode() const;

	/** Sets if the interactor can carry an object */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API void SetCanCarry(const bool bInCanCarry);

	/** Gets if the interactor can carry an object */
	UFUNCTION(BlueprintCallable, Category = "Interactor")
	UE_API bool CanCarry() const;

protected:

	/** To be overridden. Called by HandleInputKey before delegates and default input implementation */
	virtual void PreviewInputKey( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const EInputEvent Event, bool& bOutWasHandled ) {};

	/** To be overridden. Called by HandleInputAxis before delegates and default input implementation */
	virtual void PreviewInputAxis( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const float Delta, const float DeltaTime, bool& bOutWasHandled ) {};

	/** To be overridden. Called by HandleInputKey before delegates and default input implementation */
	virtual void HandleInputKey( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const EInputEvent Event, bool& bOutWasHandled ) {};

	/** To be overridden. Called by HandleInputKey before delegates and default input implementation */
	UFUNCTION(BlueprintImplementableEvent, CallInEditor, meta = (DisplayName = "HandleInputKey"), Category = "Interactor")
	UE_API void HandleInputKey_BP( const FViewportActionKeyInput& Action, const FKey Key, const EInputEvent Event, bool& bOutWasHandled);

	/** To be overridden. Called by HandleInputAxis before delegates and default input implementation */
	virtual void HandleInputAxis( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const float Delta, const float DeltaTime, bool& bOutWasHandled ) {};

	/** To be overridden. Called by HandleInputAxis before delegates and default input implementation */
	UFUNCTION(BlueprintImplementableEvent, CallInEditor, meta = (DisplayName = "HandleInputAxis"), Category = "Interactor")
	UE_API void HandleInputAxis_BP( const FViewportActionKeyInput& Action, const FKey Key, const float Delta, const float DeltaTime, bool& bOutWasHandled);

	/** If this interactors allows to smooth the laser. Default is true. */
	UE_API virtual bool AllowLaserSmoothing() const;

protected:

	/** All the private data for the interactor */
	FViewportInteractorData InteractorData;

	/** Mapping of raw keys to actions*/
	UPROPERTY()
	TMap<FKey, FViewportActionKeyInput> KeyToActionMap;

	/** The owning world interaction */
	UPROPERTY()
	TObjectPtr<class UViewportWorldInteraction> WorldInteraction;

	/** The paired interactor by the world interaction */
	UPROPERTY()
	TObjectPtr<UViewportInteractor> OtherInteractor;

	/** True if this interactor supports 'grabber sphere' interaction.  Usually disabled for mouse cursors */
	bool bAllowGrabberSphere;

	/** True if this interactor can 'carry' objects in VR, that is, translation and rotation of the interactor is inherited by the object, instead of just translation */
	bool bCanCarry;

	/** Store end of the laser pointer. This will be returned when calling GetLaserPointer multiple times a tick */
	TOptional<FVector> SavedLaserPointerEnd;

	/** Store the last hitresult from the laser, to use that when calling GetHitResultFromLaserPointer multiple times in a tick. */
	TOptional<FHitResult> SavedHitResult;

	/** Store the last hitresult from the laser, to use that when calling GetHitResultFromLaserPointer multiple times in a tick. */
	TOptional<EHitResultGizmoFilterMode> SavedHitResultFilterMode;

private:

	EHitResultGizmoFilterMode CurrentHitResultGizmoFilterMode;

	/** Smoothing filter for laser */
	ViewportInteractionUtils::FOneEuroFilter SmoothingOneEuroFilter;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
