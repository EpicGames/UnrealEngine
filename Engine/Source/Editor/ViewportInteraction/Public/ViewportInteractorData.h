// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "Components/WidgetComponent.h"
#include "ViewportInteractionTypes.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


/** Represents a single virtual hand */
struct UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FViewportInteractorData
{
	//
	// Positional data
	//

	/** Your hand (laser origin) in the virtual world in world space, usually driven by VR motion controllers */
	FTransform Transform;

	/** Your hand transform, in the local tracking space (unaffected by world scaling) */
	FTransform RoomSpaceTransform;

	/** Hand transform in world space from the previous frame */
	FTransform LastTransform;

	/** Room space hand transform from the previous frame */
	FTransform LastRoomSpaceTransform;

	//
	// Hover feedback
	//

	/** The widget component we last hovered over.  This is used to detect when the laser pointer moves over or leaves a widget, and is not reset every frame */
	TWeakObjectPtr<UWidgetComponent> LastHoveredWidgetComponent; //@todo: ViewportInteraction: UI should not be in this module
		
	/** Position the laser pointer impacted an interactive object at (UI, meshes, etc.) */
	TOptional<FVector> HoverLocation;

	/** The current component hovered by the laser pointer of this hand */
	TWeakObjectPtr<class UActorComponent> LastHoveredActorComponent;

	/** The last location that we hovered over UI at in the world.  This is used for dragging and dropping from UI that
	    may have already been closed, such as Content Browser */
	FVector LastHoverLocationOverUI;

	//
	// General input
	//

	/** True if we're currently holding the 'SelectAndMove' button down after clicking on an actor */
	TWeakObjectPtr<UActorComponent> ClickingOnComponent;


	//
	// Object/world movement
	// 

	/** What we're currently dragging with this hand, if anything */
	EViewportInteractionDraggingMode DraggingMode;

	/** What we were doing last.  Used for inertial movement. */
	EViewportInteractionDraggingMode LastDraggingMode;

	/** True if we're dragging using the grabber sphere, or false if we're using the laser (or world movement) */
	bool bDraggingWithGrabberSphere;

	/** True if this is the first update since we started dragging */
	bool bIsFirstDragUpdate;

	/** True if we were assisting the other hand's drag the last time we did anything.  This is used for inertial movement */
	bool bWasAssistingDrag;

	/** Length of the ray that's dragging */
	double DragRayLength;

	/** Location that we dragged to last frame (end point of the ray) */
	FVector LastDragToLocation;

	/** The transform of the interactor when we first started the drag */
	FTransform InteractorTransformAtDragStart;

	/** Where the grabber sphere center point was when we first started the drag */
	FVector GrabberSphereLocationAtDragStart;

	/** Grabber sphere or laser pointer impact location at the drag start */
	FVector ImpactLocationAtDragStart;

	/** How fast to move selected objects every frame for inertial translation */
	FVector DragTranslationVelocity;

	/** How fast to adjust ray length every frame for inertial ray length changes */
	double DragRayLengthVelocity;

	/** While dragging, true if we're dragging at least one simulated object that we're driving the velocities of.  When this
	    is true, our default inertia system is disabled and we rely on the physics engine to take care of inertia */
	bool bIsDrivingVelocityOfSimulatedTransformables;


	//
	// Transform gizmo interaction
	//

	/** Where the gizmo was placed at the beginning of the current interaction */
	FTransform GizmoStartTransform;

	/** Where the gizmo was last frame.  This is used for interpolation and smooth snapping */
	FTransform GizmoLastTransform;

	/** Where the gizmo wants to be right now, with snaps applied. */
	FTransform GizmoTargetTransform;

	/** Where the gizmo wants to be right now, if no snaps were applied.  This is used for interpolation and smooth snapping */
	FTransform GizmoUnsnappedTargetTransform;

	/** A transform that we're interpolating from, toward the target transform.  This is used when placing objects, so they'll smoothly interpolate to their initial location. */
	FTransform GizmoInterpolationSnapshotTransform;

	/** Our gizmo bounds at the start of the interaction, in actor local space. */
	FBox GizmoStartLocalBounds;

	ELockedWorldDragMode LockedWorldDragMode;
	float GizmoScaleSinceDragStarted;
	float GizmoRotationRadiansSinceDragStarted;

	/** For a single axis drag, this is the cached local offset where the laser pointer ray intersected the axis line on the first frame of the drag */
	FVector GizmoSpaceFirstDragUpdateOffsetAlongAxis;

	/** When dragging with an axis/plane constraint applied, this is the difference between the actual "delta from start" and the constrained "delta from start".
		This is used when the user releases the object and inertia kicks in */
	FVector GizmoSpaceDragDeltaFromStartOffset;

	/** The gizmo interaction we're doing with this hand */
	TWeakObjectPtr<class UViewportDragOperationComponent> DragOperationComponent;
	
	/** The last drag operation. */
	class UViewportDragOperation* LastDragOperation;

	/** Which handle on the gizmo we're interacting with, if any */
	TOptional<FTransformGizmoHandlePlacement> OptionalHandlePlacement;

	/** The gizmo component we're dragging right now */
	TWeakObjectPtr<class USceneComponent> DraggingTransformGizmoComponent;

	/** Gizmo component that we're hovering over, or nullptr if not hovering over any */
	TWeakObjectPtr<class USceneComponent> HoveringOverTransformGizmoComponent;

	/** Gizmo handle that we hovered over last (used only for avoiding spamming of hover haptics!) */
	TWeakObjectPtr<class USceneComponent> HoverHapticCheckLastHoveredGizmoComponent;

	/** If the latest hitresult is hovering over a priority type */
	bool bHitResultIsPriorityType;

	/** The offset between the hitlocation of the object selected to start dragging and its center. 
		This is used to offset the objects when dragging to the end of the laser */
	FVector StartHitLocationToTransformableCenter;

	/** Default constructor for FVirtualHand that initializes safe defaults */
	FViewportInteractorData()
	{
		LastHoveredWidgetComponent = nullptr;
		HoverLocation = TOptional<FVector>();
		LastHoveredActorComponent = nullptr;
		LastHoverLocationOverUI = FVector::ZeroVector;

		ClickingOnComponent = nullptr;

		DraggingMode = EViewportInteractionDraggingMode::Nothing;
		bDraggingWithGrabberSphere = false;
		LastDraggingMode = EViewportInteractionDraggingMode::Nothing;
		bWasAssistingDrag = false;
		bIsFirstDragUpdate = false;
		DragRayLength = 0.0f;
		LastDragToLocation = FVector::ZeroVector;
		InteractorTransformAtDragStart = FTransform::Identity;
		GrabberSphereLocationAtDragStart = FVector::ZeroVector;
		ImpactLocationAtDragStart = FVector::ZeroVector;
		DragTranslationVelocity = FVector::ZeroVector;
		DragRayLengthVelocity = 0.0f;
		bIsDrivingVelocityOfSimulatedTransformables = false;

		GizmoStartTransform = FTransform::Identity;
		GizmoStartLocalBounds = FBox(ForceInit);
		GizmoLastTransform = GizmoTargetTransform = GizmoUnsnappedTargetTransform = GizmoInterpolationSnapshotTransform = GizmoStartTransform;
		GizmoSpaceFirstDragUpdateOffsetAlongAxis = FVector::ZeroVector;
		GizmoSpaceDragDeltaFromStartOffset = FVector::ZeroVector;
		DragOperationComponent.Reset();
		LastDragOperation = nullptr;
		LockedWorldDragMode = ELockedWorldDragMode::Unlocked;
		GizmoScaleSinceDragStarted = 0.0f;
		GizmoRotationRadiansSinceDragStarted = 0.0f;
		OptionalHandlePlacement.Reset();
		DraggingTransformGizmoComponent = nullptr;
		HoveringOverTransformGizmoComponent = nullptr;
		HoverHapticCheckLastHoveredGizmoComponent = nullptr;

		bHitResultIsPriorityType = false;

		StartHitLocationToTransformableCenter = FVector::ZeroVector;
	}
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
