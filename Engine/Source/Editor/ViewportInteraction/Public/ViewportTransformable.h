// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Math/Box.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "Math/Vector.h"

/**
 * Represents an object that we're actively interacting with, such as a selected actor
 */
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") FViewportTransformable
{

public:

	/** Sets up safe defaults */
	FViewportTransformable()
		: StartTransform( FTransform::Identity )
	{
	}

	virtual ~FViewportTransformable()
	{
	}

	/** Gets the current transform of this object */
	virtual const FTransform GetTransform() const = 0;

	/** Updates the transform of the actual object */
	virtual void ApplyTransform( const FTransform& NewTransform, const bool bSweep ) = 0;

	/** Returns the bounding box of this transformable, built in the specified coordinate system */
	virtual FBox BuildBoundingBox( const FTransform& BoundingBoxToWorld ) const = 0;

	/** Returns true if this transformable is a single, unoriented point in space, thus never
	    supports being rotated or scaled when only a single transformable is selected. */
	virtual bool IsUnorientedPoint() const
	{
		return false;
	}

	/** Returns true if this transformable is a physically simulated kinematic object */
	virtual bool IsPhysicallySimulated() const
	{
		return false;
	}

	/** Returns true if this transformable should be 'carried' (moved and rotated) when dragged, if possible, instead of only translated */
	virtual bool ShouldBeCarried() const
	{
		return false;
	}

	/** For physically simulated objects, sets the new velocity of the object */
	virtual void SetLinearVelocity( const FVector& NewVelocity )
	{
	}

	/** Get the velocity of the object */
	virtual FVector GetLinearVelocity() const
	{
		return FVector::ZeroVector;
	}

	/** For actor transformables, this will add it's actor to the incoming list */
	virtual void UpdateIgnoredActorList( TArray<class AActor*>& IgnoredActors )
	{
	}


	/** The object's world space transform when we started the action */
	FTransform StartTransform;
};

