// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ViewportInteractor.h"
#include "ViewportWorldInteraction.h"
#include "ViewportTransformer.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UCLASS(MinimalAPI,  abstract )
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UViewportTransformer : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION()
	UE_API virtual void Init( UViewportWorldInteraction* InitViewportWorldInteraction );

	UFUNCTION()
	UE_API virtual void Shutdown();

	/** @return If this transformer can be used to align its transformable to actors in the scene */
	UFUNCTION()
	virtual bool CanAlignToActors() const
	{
		return false;
	};

	/** @return True if the transform gizmo should be aligned to the center of the bounds of all selected objects with more than one is selected.  Otherwise it will be at the pivot of the last transformable, similar to legacl editor actor selection */
	UFUNCTION()
	virtual bool ShouldCenterTransformGizmoPivot() const
	{
		return false;
	}

	/** When starting to drag */
	UFUNCTION()
	virtual void OnStartDragging(UViewportInteractor* Interactor) {};

	/** When ending drag */
	UFUNCTION()
	virtual void OnStopDragging(UViewportInteractor* Interactor) {};

protected:

	/** The viewport world interaction object we're registered with */
	UPROPERTY()
	TObjectPtr<UViewportWorldInteraction> ViewportWorldInteraction;

};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
