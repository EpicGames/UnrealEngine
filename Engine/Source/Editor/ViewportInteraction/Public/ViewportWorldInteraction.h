// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtr.h"
#include "GameFramework/Actor.h"
#include "EditorViewportClient.h"
#include "Misc/App.h"
#include "ViewportInteractionTypes.h"
#include "EditorWorldExtension.h"
#include "VIBaseTransformGizmo.h"
#include "LevelEditorViewport.h"
#include "ViewportTransformable.h"
#include "GenericPlatform/ICursor.h"
#include "ViewportWorldInteraction.generated.h"

#define UE_API VIEWPORTINTERACTION_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") ViewportWorldActionTypes
{
	static const FName NoAction( "NoAction" );
	static const FName WorldMovement( "WorldMovement" );
	static const FName SelectAndMove( "SelectAndMove" );
	static const FName SelectAndMove_FullyPressed( "SelectAndMove_FullyPressed" );
	static const FName Undo( "Undo" );
	static const FName Redo( "Redo" );
	static const FName Delete( "Delete" );
}

// Forward declare the GizmoHandleTypes
enum class EGizmoHandleTypes : uint8;
class IViewportInteractableInterface;
class UViewportInteractionAssetContainer;
class UViewportInteractor;

UENUM()
enum class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") EViewportWorldInteractionType : uint8
{
	VR = 0,
	Legacy = 1
};

UCLASS(MinimalAPI, BlueprintType, Transient)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") UViewportWorldInteraction : public UEditorWorldExtension
{
	GENERATED_BODY()

public:

	UE_API UViewportWorldInteraction();
	// UEditorExtension overrides
	UE_API virtual void Init() override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void Tick( float DeltaSeconds ) override;

	/** Initialize colors */
	UE_API void InitColors();

	/** Adds interactor to the worldinteraction */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API void AddInteractor( UViewportInteractor* Interactor );

	/** Removes interactor from the worldinteraction and removes the interactor from its paired interactor if any */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API void RemoveInteractor( UViewportInteractor* Interactor );

	/** Creates an interactor for the mouse cursor.  If one already exists, this will add a reference to it.  Remember to
	    call ReleaseMouseCursorInteractor() when you're done. */
	UE_API void AddMouseCursorInteractor();

	/** When you're finished with the mouse cursor interactor, call this to release it.  It will be destroyed unless
	    another system is still using it */
	UE_API void ReleaseMouseCursorInteractor();

	/** Gets the event for hovering update which is broadcasted for each interactor */
	DECLARE_EVENT_ThreeParams( UViewportWorldInteraction, FOnVIHoverUpdate, class UViewportInteractor* /* Interactor */, FVector& /* OutHoverImpactPoint */, bool& /* bWasHandled */ );
	FOnVIHoverUpdate& OnViewportInteractionHoverUpdate() { return OnHoverUpdateEvent; }

	/** Gets the event for previewing input actions.  This allows systems to get a first chance at actions before everything else. */
	DECLARE_EVENT_FiveParams( UViewportWorldInteraction, FOnPreviewInputAction, class FEditorViewportClient& /* ViewportClient */, class UViewportInteractor* /* Interactor */, const struct FViewportActionKeyInput& /* Action */, bool& /* bOutIsInputCaptured */, bool& /* bWasHandled */ );
	FOnPreviewInputAction& OnPreviewInputAction() { return OnPreviewInputActionEvent; }

	/** Gets the event for input actions update which is broadcasted for each interactor */
	DECLARE_EVENT_FiveParams( UViewportWorldInteraction, FOnVIActionHandle, class FEditorViewportClient& /* ViewportClient */, class UViewportInteractor* /* Interactor */, const struct FViewportActionKeyInput& /* Action */, bool& /* bOutIsInputCaptured */, bool& /* bWasHandled */ );
	FOnVIActionHandle& OnViewportInteractionInputAction() { return OnInputActionEvent; }

	/** Called when the user clicks on 'nothing' to allow systems to deselect anything that's selected */
	DECLARE_EVENT_ThreeParams( UViewportWorldInteraction, FOnViewportInteractionInputUnhandled, class FEditorViewportClient& /* ViewportClient */, class UViewportInteractor* /* Interactor */, const struct FViewportActionKeyInput& /* Action */ );
	FOnViewportInteractionInputUnhandled& OnViewportInteractionInputUnhandled() { return OnViewportInteractionInputUnhandledEvent; };

	/** To handle raw key input from the Input Preprocessor */
	DECLARE_EVENT_FourParams( UViewportWorldInteraction, FOnHandleInputKey, class FEditorViewportClient* /* ViewportClient */, const FKey /* Key */, const EInputEvent /* Event */, bool& /* bWasHandled */ );
	FOnHandleInputKey& OnHandleKeyInput() { return OnKeyInputEvent; }

	/** To handle raw axis input from the Input Preprocessor */
	DECLARE_EVENT_SixParams( UViewportWorldInteraction, FOnHandleInputAxis, class FEditorViewportClient* /* ViewportClient */, const int32 /* ControllerId */, const FKey /* Key */, const float /* Delta */, const float /* DeltaTime */, bool& /* bWasHandled */ );
	FOnHandleInputAxis& OnHandleAxisInput() { return OnAxisInputEvent; }

	/** Gets the event for when an interactor starts dragging */
	DECLARE_EVENT_OneParam( UViewportWorldInteraction, FOnStartDragging, class UViewportInteractor* /** Interactor */ );
	FOnStartDragging& OnStartDragging() { return OnStartDraggingEvent; };

	/** Gets the event for when an interactor stops dragging */
	DECLARE_EVENT_OneParam( UViewportWorldInteraction, FOnStopDragging, class UViewportInteractor* /** Interactor */ );
	FOnStopDragging& OnStopDragging() { return OnStopDraggingEvent; };

	/** Gets the event for when the current set of transformables has finally stop moving (after a drag, and all inertial effects or snapping interpolation has completed */
	DECLARE_EVENT( UViewportWorldInteraction, FOnFinishedMovingTransformables );
	FOnFinishedMovingTransformables& OnFinishedMovingTransformables() { return OnFinishedMovingTransformablesEvent; };

	DECLARE_EVENT_OneParam( UViewportWorldInteraction, FOnViewportWorldInteractionTick, const float /* DeltaTime */ );
	FOnViewportWorldInteractionTick& OnPreWorldInteractionTick() { return OnPreWorldInteractionTickEvent; };
	FOnViewportWorldInteractionTick& OnPostWorldInteractionTick() { return OnPostWorldInteractionTickEvent; };

	/** Sets the system that should be used to transform objects in the scene.  Only one can be active at a time.  If you pass
	    nullptr, a default transformer for actors/components will be enabled */
	UE_API void SetTransformer( class UViewportTransformer* NewTransformer );

	/** @return	Gets the currently active transformer object */
	const UViewportTransformer* GetTransformer() const
	{
		return ViewportTransformer;
	}

	/** Sets the list of objects that this system will be responsible for transforming when interacting using the
	    gizmo or directly on the objects */
	UE_API void SetTransformables( TArray< TUniquePtr< FViewportTransformable > >&& NewTransformables, const bool bNewObjectsSelected );

	/** When using VR, this sets the viewport client that's been "possessed" by the head mounted display.  Only valid when VR is enabled. */
	UE_API void SetDefaultOptionalViewportClient( const TSharedPtr<class FEditorViewportClient>& InEditorViewportClient );

	/** Pairs to interactors by setting	the other interactor for each interactor */
	UE_API void PairInteractors( UViewportInteractor* FirstInteractor, UViewportInteractor* SecondInteractor );

	/**
	 * Adds an actor to the list of actors to never allow an interactor to hit in the scene.  No selection.  No hover.
	 * There's no need to remove actors from this list.  They'll expire from it automatically when destroyed.
	 *
	 * @param	ActorToExcludeFromHitTests	The actor that should be forever excluded from hit tests
	 */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API void AddActorToExcludeFromHitTests( AActor* ActorToExcludeFromHitTests );

	/** @return	Returns the list of actors that should never be included in interctor hit tests (hovering or selection) */
	const TArray<TWeakObjectPtr<AActor>>& GetActorsToExcludeFromHitTest() const
	{
		return ActorsToExcludeFromHitTest;
	}

	//
	// Input
 	//

	UE_API virtual bool InputKey( FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event ) override;
	UE_API virtual bool InputAxis( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime ) override;

	/** Handles the key input from the Input Preprocessor*/
	UE_API bool PreprocessedInputKey( const FKey Key, const EInputEvent Event );

	/** Handles the axis input from the Input Preprocessor*/
	UE_API bool PreprocessedInputAxis( const int32 ControllerId, const FKey Key, const float Delta, const double DeltaTime );

	/** Gets the world space transform of the calibrated VR room origin.  When using a seated VR device, this will feel like the
	camera's world transform (before any HMD positional or rotation adjustments are applied.) */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API FTransform GetRoomTransform() const;

	/** Gets the transform of the viewport / user's HMD in room space */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API FTransform GetRoomSpaceHeadTransform() const;

	/** Gets the transform of the viewport / user's HMD in world space */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API FTransform GetHeadTransform() const;

	/** Sets a new transform for the room so that the HMD is aligned to the new transform.
		The Head is kept level to the ground and only rotated on the yaw */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API void SetHeadTransform( const FTransform& NewHeadTransform );

	/** Returns true if we actually are using VR and have a valid head location, meaning a call to GetHeadTransform() is valid */
	UE_API bool HaveHeadTransform() const;

	/** Sets a new transform for the room, in world space.  This is basically setting the editor's camera transform for the viewport */
	UE_API void SetRoomTransform( const FTransform& NewRoomTransform );

	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API void SetRoomTransformForNextFrame(const FTransform& NewRoomTransform);

	/** Gets the world scale factor, which can be multiplied by a scale vector to convert to room space */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API float GetWorldScaleFactor() const;

	/** Gets the currently used viewport */
	UE_API FEditorViewportClient* GetDefaultOptionalViewportClient() const;

	/** Returns the Unreal controller ID for the motion controllers we're using */
	int32 GetMotionControllerID() const
	{
		return MotionControllerID;
	}

	/** Invokes the editor's undo system to undo the last change */
	UE_API void Undo();

	/** Redoes the last change */
	UE_API void Redo();

	/** Deletes all the selected objects */
	UE_API void DeleteSelectedObjects();

	/** Copies selected objects to the clipboard */
	UE_API void Copy();

	/** Pastes the clipboard contents into the scene */
	UE_API void Paste();

	/** Duplicates the selected objects */
	UE_API void Duplicate();

	/** Deselects all the current selected objects */
	UE_API void Deselect();

	/** Called to finish a drag action with the specified interactor */
	UE_API void StopDragging( class UViewportInteractor* Interactor );

	/** Starts dragging selected objects around.  Called when clicking and dragging on actors/gizmos in the world, or when placing new objects. */
	UE_API void StartDragging( UViewportInteractor* Interactor, UActorComponent* ClickedTransformGizmoComponent, const FVector& HitLocation, const bool bIsPlacingNewObjects, const bool bAllowInterpolationWhenPlacing, const bool bShouldUseLaserImpactDrag, const bool bStartTransaction, const bool bWithGrabberSphere );

	DECLARE_EVENT_OneParam( UViewportWorldInteraction, FOnWorldScaleChanged, const float /* NewWorldToMetersScale */);
	virtual FOnWorldScaleChanged& OnWorldScaleChanged() { return OnWorldScaleChangedEvent; };

	/** Sets which transform gizmo coordinate space is used */
	UE_API void SetTransformGizmoCoordinateSpace( const ECoordSystem NewCoordSystem );

	/** Returns which transform gizmo coordinate space we're using, world or local */
	UE_API ECoordSystem GetTransformGizmoCoordinateSpace() const;

	/** Returns the maximum user scale */
	UE_API float GetMaxScale();

	/** Returns the minimum user scale */
	UE_API float GetMinScale();

	/** Sets GNewWorldToMetersScale */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API void SetWorldToMetersScale( const float NewWorldToMetersScale, const bool bCompensateRoomWorldScale = false );

	/** Tells the world interaction system to skip updates of world movement this frame.  This is useful if you've called
	    SetWorldToMetersScale() yourself to set a static world scale, and the cached room-space transforms for the last
		frame will no longer make sense */
	void SkipInteractiveWorldMovementThisFrame()
	{
		bSkipInteractiveWorldMovementThisFrame = true;
	}

	/** Checks to see if we're allowed to interact with the specified component */
	UE_API bool IsInteractableComponent( const UActorComponent* Component ) const;

	/** Gets the transform gizmo actor, or returns null if we currently don't have one */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API class ABaseTransformGizmo* GetTransformGizmoActor();

	/** Sets whether the transform gizmo should be visible at all */
	UE_API void SetTransformGizmoVisible( const bool bShouldBeVisible );

	/** Returns whether the transform gizmo should be visible */
	UE_API bool ShouldTransformGizmoBeVisible() const;

	/** Returns whether the transform gizmo is actually visible right now.  This is different than whether it 'should' be
	    visible, as the gizmo can be implicitly hidden if there are no selected objects, for example. */
	UE_API bool IsTransformGizmoVisible() const;

	/** Sets how large the transform gizmo is */
	UE_API void SetTransformGizmoScale( const float NewScale );

	/** Returns the current transform gizmo scale */
	UE_API float GetTransformGizmoScale() const;

	/** Sets if objects are dragged with either hand since last time selecting something */
	UE_API void SetDraggedSinceLastSelection( const bool bNewDraggedSinceLastSelection );

	/** Sets the transform of the gizmo when starting drag */
	UE_API void SetLastDragGizmoStartTransform( const FTransform NewLastDragGizmoStartTransform );

	/** Gets all the interactors */
	UFUNCTION(BlueprintCallable, Category = "ViewportWorldInteraction")
	UE_API const TArray<UViewportInteractor*>& GetInteractors() const;

	/** Given a world space velocity vector, applies inertial damping to it to slow it down */
	UE_API void ApplyVelocityDamping( FVector& Velocity, const bool bVelocitySensitive );

	/** Gets the current Gizmo handle type */
	UE_API EGizmoHandleTypes GetCurrentGizmoType() const;

	/** Sets the current gizmo handle type */
	UE_API void SetGizmoHandleType( const EGizmoHandleTypes InGizmoHandleType );

	/** Changes the transform gizmo class that it will change to next refresh */
	UE_API void SetTransformGizmoClass( const TSubclassOf<ABaseTransformGizmo>& NewTransformGizmoClass );

	/** Sets the currently dragged interactavle */
	UE_API void SetDraggedInteractable( IViewportInteractableInterface* InDraggedInteractable, UViewportInteractor* Interactor );

	/** Check if there is another interactor hovering over the component */
	UE_API bool IsOtherInteractorHoveringOverComponent( UViewportInteractor* Interactor, UActorComponent* Component ) const;

	/** Switch which transform gizmo coordinate space we're using. */
	UE_API void CycleTransformGizmoCoordinateSpace();

	/** Figures out where to place an object when tracing it against the scene using a laser pointer */
	UE_API bool FindPlacementPointUnderLaser( UViewportInteractor* Interactor, FVector& OutHitLocation );

	/** Gets the tracking transactions */
	UE_API FTrackingTransaction& GetTrackingTransaction();

	/** Sets whether we should allow input events from the input preprocessor or not */
	UE_API void SetUseInputPreprocessor( bool bInUseInputPreprocessor );

	/** Gets a list of all of the objects we're currently interacting with, such as selected actors */
	TArray< TUniquePtr< FViewportTransformable > >& GetTransformables()
	{
		return Transformables;
	}

	/** The ability to move and scale the world */
	UE_API void AllowWorldMovement( bool bAllow );

	/** For other systems to check if the Viewport World Interaction system is currently aligning transformables to actors*/
	UE_API bool AreAligningToActors() const;
	/** For other systems to check if the Viewport World Interaction system currently has candidate actors selected */
	UE_API bool HasCandidatesSelected();
	/** If there are no currently selected candidates, use the currently selected actors as candidates. Otherwise, reset the candidates */
	UE_API void SetSelectionAsCandidates();

	/** Gets the current delta time, so functions that don't get the delta time passed can still get it */
	UE_API float GetCurrentDeltaTime() const;

	/** Getters and setters for whether or not to show the cursor on the viewport */
	UE_API bool ShouldSuppressExistingCursor() const;
	void SetShouldSuppressExistingCursor(const bool bInShouldSuppressCursor)
	{
		bShouldSuppressCursor = bInShouldSuppressCursor;
	};

	/** Getters and setters for whether or not to show the cursor on the viewport */
	UE_API bool ShouldForceCursor() const;
	UE_API void SetForceCursor(const bool bInShouldForceCursor);

	/** Gets the container for all the assets of ViewportInteraction. */
	UE_API const UViewportInteractionAssetContainer& GetAssetContainer() const;

	/** Static function to load the asset container */
	static UE_API const UViewportInteractionAssetContainer* LoadAssetContainer();

	/** Plays sound at location. */
	UE_API void PlaySound(USoundBase* SoundBase, const FVector& InWorldLocation, const float InVolume = 1.0f);

	/** Set if this world interaction is in VR. */
	UE_API void SetInVR(const bool bInVR);

	/** Get if this world interaction is in VR. */
	UE_API bool IsInVR() const;

	UE_API FVector SnapLocation(const bool bLocalSpaceSnapping, const FVector& DesiredGizmoLocation, const FTransform &GizmoStartTransform, const FVector SnapGridBase, const bool bShouldConstrainMovement, const FVector AlignAxes);

	/** Forces the VWI to fall back to standard desktop interactions */
	UE_API void UseLegacyInteractions();

	/** Sets the VWI to use its own interactions */
	UE_API void UseVWInteractions();

protected:

	UE_API virtual void TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState) override;
	UE_API virtual void EnteredSimulateInEditor() override;
	UE_API virtual void LeftSimulateInEditor(UWorld* SimulateWorld) override;

private:

	/** Called when the editor is closed */
	UE_API void OnEditorClosed();

	/** Called every frame to update hover state */
	UE_API void HoverTick( const float DeltaTime );

	/** Updates the interaction */
	UE_API void InteractionTick( const float DeltaTime );

	/** Called by the world interaction system when one of our components is dragged by the user.  If null is
	    passed in then we'll treat it as dragging the whole object (rather than a specific axis/handle) */
	UE_API void UpdateDragging(
		const float DeltaTime,
		bool& bIsFirstDragUpdate,
		UViewportInteractor* Interactor,
		const EViewportInteractionDraggingMode DraggingMode,
		class UViewportDragOperation* DragOperation,
		const bool bWithTwoHands,
		const TOptional<FTransformGizmoHandlePlacement> OptionalHandlePlacement,
		const FVector& DragDelta,
		const FVector& OtherHandDragDelta,
		const FVector& DraggedTo,
		const FVector& OtherHandDraggedTo,
		const FVector& DragDeltaFromStart,
		const FVector& OtherHandDragDeltaFromStart,
		const FVector& LaserPointerStart,
		const FVector& LaserPointerDirection,
		const float LaserPointerMaxLength,
		const bool bIsLaserPointerValid,
		const FTransform& GizmoStartTransform,
		FTransform& GizmoLastTransform,
		FTransform& GizmoTargetTransform,
		FTransform& GizmoUnsnappedTargetTransform,
		const FTransform& GizmoInterpolationSnapshotTransform,
		const FBox& GizmoStartLocalBounds,
		const USceneComponent* const DraggingTransformGizmoComponent,
		FVector& GizmoSpaceFirstDragUpdateOffsetAlongAxis,
		FVector& DragDeltaFromStartOffset,
		ELockedWorldDragMode& LockedWorldDragMode,
		float& GizmoScaleSinceDragStarted,
		float& GizmoRotationRadiansSinceDragStarted,
		bool& bIsDrivingVelocityOfSimulatedTransformables,
		FVector& OutUnsnappedDraggedTo);

	/** Given a drag delta from a starting point, contrains that delta based on a gizmo handle axis */
	UE_API FVector ComputeConstrainedDragDeltaFromStart(
		const bool bIsFirstDragUpdate,
		const bool bOnPlane,
		const TOptional<FTransformGizmoHandlePlacement> OptionalHandlePlacement,
		const FVector& DragDeltaFromStart,
		const FVector& LaserPointerStart,
		const FVector& LaserPointerDirection,
		const bool bIsLaserPointerValid,
		const FTransform& GizmoStartTransform,
		const float LaserPointerMaxLength,
		FVector& GizmoSpaceFirstDragUpdateOffsetAlongAxis,
		FVector& DragDeltaFromStartOffset,
		FVector& OutClosestPointOnLaser) const;

	/** Called after dragging has finished with one or both hands and all inertial movement and interpolation has finished with
	    the set of transformables we were working with.  Will also be called if the drag was interrupted. */
	UE_API void FinishedMovingTransformables();

	/** Checks to see if smooth snapping is currently enabled */
	UE_API bool IsSmoothSnappingEnabled() const;

	/** Returns the time since this was last entered */
	inline FTimespan GetTimeSinceEntered() const
	{
		return FTimespan::FromSeconds( FApp::GetCurrentTime() ) - AppTimeEntered;
	}

	/** Conditionally polls input from interactors, if we haven't done that already this frame */
	UE_API void PollInputIfNeeded();

	/** Refreshes the transform gizmo to match the currently selected actor set */
	UE_API void RefreshTransformGizmo( const bool bNewObjectsSelected );

	/** Spawn new transform gizmo when refreshing and at start if there is none yet or the current one is different from the TransformGizmoClass */
	UE_API void SpawnTransformGizmoIfNeeded();

	/** Gets the inertie contribution of another interactor */
	UE_API UViewportInteractor* GetOtherInteractorIntertiaContribute( UViewportInteractor* Interactor );

	/** Destroys the actors */
	UE_API void DestroyActors();

	/** Gets the snap grid mesh MID */
	class UMaterialInstanceDynamic* GetSnapGridMID()
	{
		SpawnGridMeshActor();
		return SnapGridMID;
	}

	/** Spawns the grid actor */
	UE_API void SpawnGridMeshActor();

	/** Average location of all the current transformables */
	UE_API FVector CalculateAverageLocationOfTransformables();

	/** Location the transformable should snap to if aligning to other transformables*/
	UE_API FVector FindTransformGizmoAlignPoint(const FTransform& GizmoStartTransform, const FTransform& DesiredGizmoTransform, const bool bShouldConstrainMovement, FVector ConstraintAxes);
	UE_API void DrawBoxBrackets(const FBox InActor, const FTransform LocalToWorld, const FLinearColor BracketColor);

	/** Calculate a new room transform location according to the world to meters scale */
	UE_API void CompensateRoomTransformForWorldScale(FTransform& InOutRoomTransform, const float InNewWorldToMetersScale, const FVector& InRoomPivotLocation);

	/** If there is a transformable with velocity in simulate */
	UE_API bool HasTransformableWithVelocityInSimulate() const;

	/** Get mode tools from the viewport client. */
	UE_API class FEditorModeTools& GetModeTools() const;

	//
	// Colors
	//
public:
	enum EColors
	{
		DefaultColor,
		Forward,
		Right,
		Up,
		GizmoHover,
		GizmoDragging,
		TotalCount
	};

	/** Gets the color from color type */
	UE_API FLinearColor GetColor(const EColors Color, const float Multiplier = 1.f) const;

private:

	/** The path of the asset container */
	static UE_API const TCHAR* AssetContainerPath;

	// All the colors for this mode
	TArray<FLinearColor> Colors;

	/** Candidate actors for aligning in the scene */
	TArray<const AActor*> CandidateActors;

protected:

	/** True if we've dragged objects with either hand since the last time we selected something */
	bool bDraggedSinceLastSelection;

	/** Gizmo start transform of the last drag we did with either hand.  Only valid if bDraggedSinceLastSelection is true */
	FTransform LastDragGizmoStartTransform;

	//
	// Undo/redo
	//

	/** Manages saving undo for selected actors while we're dragging them around */
	FTrackingTransaction TrackingTransaction;

private:

	/** App time that we entered this */
	FTimespan AppTimeEntered;

	/** All the interactors registered to modify the world */
	UPROPERTY()
	TArray< TObjectPtr<class UViewportInteractor> > Interactors;

	/** The active system being used to transform objects */
	UPROPERTY()
	TObjectPtr<class UViewportTransformer> ViewportTransformer;

	//
	// Viewport
	//

	/** The viewport that the worldinteraction is used for */
	class FEditorViewportClient* DefaultOptionalViewportClient;

	/** The last frame that input was polled.  We keep track of this so that we can make sure to poll for latest input right before
	its first needed each frame */
	uint32 LastFrameNumberInputWasPolled;

	/** The Unreal controller ID for the motion controllers we're using */
	int32 MotionControllerID;


	//
	// World movement
	//

	/** The world to meters scale before we changed it (last frame's world to meters scale) */
	float LastWorldToMetersScale;

	/** True if we should skip interactive world movement/rotation/scaling this frame (because it was already set by something else) */
	bool bSkipInteractiveWorldMovementThisFrame;

	/** The room transform to apply at the beginning of the next frame (if needed.)  This is used to defer
	    updates of the room transform to match with the GNewWorldToMetersScale, which will also take effect
		on the next frame.  It's critical that those two are applied at the same time, because otherwise a
		frame rate dependent feedback loop will occur */
	TOptional<TTuple<FTransform,uint32>> RoomTransformToSetOnFrame;

	/** Storage for Low speed damping value for when we're dragging ourself around the world */
	float LowSpeedInertiaDamping;

	/** Storage for High speed damping value for when we're dragging ourself around the world */
	float HighSpeedInertiaDamping;

	//
	// Hover state
	//

	/** Set of objects that are currently hovered over */
	TSet<FViewportHoverTarget> HoveredObjects;

	//
	// Object movement/placement
	//

	/** True if transformables are being moved around, including any inertial movement or movement caused by
	    smooth snapping or placement interpolation. */
	bool bAreTransformablesMoving;

	/** True if our transformables are currently interpolating to their target location */
	bool bIsInterpolatingTransformablesFromSnapshotTransform;

	/** When interpolating from a snapshot transform, this will be true if we should freeze the destination placement location until the interpolation finishes */
	bool bFreezePlacementWhileInterpolatingTransformables;

	/** Time that we started interpolating at */
	FTimespan TransformablesInterpolationStartTime;

	/** Duration to interpolate transformables over */
	float TransformablesInterpolationDuration;

	//
	// Transform gizmo
	//

	/** Transform gizmo actor, for manipulating selected actor(s) */
	UPROPERTY()
	TObjectPtr<class ABaseTransformGizmo> TransformGizmoActor;

	/** Class used for when refreshing the transform gizmo */
	TSubclassOf<ABaseTransformGizmo> TransformGizmoClass;

	/** Current gizmo bounds in local space.  Updated every frame.  This is not the same as the gizmo actor's bounds. */
	FBox GizmoLocalBounds;

	/** Whether the gizmo should be visible or not */
	bool bShouldTransformGizmoBeVisible;

	/** How big the transform gizmo should be */
	float TransformGizmoScale;

	/** All of the objects we're currently interacting with, such as selected actors */
	TArray< TUniquePtr< FViewportTransformable > > Transformables;

	/** The current gizmo type */ //@todo ViewportInteraction: Currently this is only used for universal gizmo.
	TOptional<EGizmoHandleTypes> GizmoType;

	/** When carrying, the last position of the interactor that was carrying.  Used for smoothing motion. */
	TOptional<FTransform> LastCarryingTransform;

	//
	// Snap grid
	//

	/** Actor for the snap grid */
	UPROPERTY()
	TObjectPtr<class AActor> SnapGridActor;

	/** The plane mesh we use to draw a snap grid under selected objects */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> SnapGridMeshComponent;

	/** MID for the snap grid, so we can update snap values on the fly */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> SnapGridMID;

	//
	// Interactable
	//

	/** The current dragged interactable */
	class IViewportInteractableInterface* DraggedInteractable;

	//
	// Events
	//

	/** Event that fires every frame to update hover state based on what's under the cursor.  Set bWasHandled to true if you detect something to hover. */
	FOnVIHoverUpdate OnHoverUpdateEvent;

	/** Event for previewing input actions.  This allows systems to get a first chance at actions before everything else. */
	FOnPreviewInputAction OnPreviewInputActionEvent;

	/** Event that is fired for when a key is pressed by an interactor */
	FOnVIActionHandle OnInputActionEvent;

	/** Called when the user clicks on 'nothing' to request systems to deselect anything that's selected */
	FOnViewportInteractionInputUnhandled OnViewportInteractionInputUnhandledEvent;

	/** Event that is fired when new key input is received by the InputProcessor */
	FOnHandleInputKey OnKeyInputEvent;

	/** Event that is fired when new axis input is received by the InputProcessor */
	FOnHandleInputAxis OnAxisInputEvent;

	/** Event that is fired when an interactor starts dragging */
	FOnStartDragging OnStartDraggingEvent;

	/** Event that is fired when an interactor stopped dragging */
	FOnStopDragging OnStopDraggingEvent;

	/** Event for when the current set of transformables has finally stop moving (after a drag, and all inertial effects or snapping interpolation has completed */
	FOnFinishedMovingTransformables OnFinishedMovingTransformablesEvent;

	/** Event to tick before the worldinteraction ticks */
	FOnViewportWorldInteractionTick OnPreWorldInteractionTickEvent;

	/** Event to tick after the worldinteraction has ticked */
	FOnViewportWorldInteractionTick OnPostWorldInteractionTickEvent;

	//
	// Interactors
	//

	/** The default mouse cursor interactor, activated on demand */
	UPROPERTY()
	TObjectPtr<class UMouseCursorInteractor> DefaultMouseCursorInteractor;

	/** Reference count for the default mouse cursor interactor.  When this reaches zero, the mouse cursor interactor goes away. */
	int32 DefaultMouseCursorInteractorRefCount;

	/** List of actors which should never be hit by an interactor, such as the 'avatar mesh actor' in VR */
	UPROPERTY()
	TArray<TWeakObjectPtr<AActor>> ActorsToExcludeFromHitTest;

	//
	// VR
	//

	/** If this world interaction is in VR */
	bool bIsInVR;

	//
	// Configuration
	//

	/** Event that is fired when the world scale changes */
	FOnWorldScaleChanged OnWorldScaleChangedEvent;

	/** If this world interaction should get input from the input processor */
	bool bUseInputPreprocessor;

	/** If the user can scale and navigate around in the world */
	bool bAllowWorldMovement;

	/** The delta time of this tick */
	float CurrentDeltaTime;

	/** Whether or not to show the cursor on the viewport */
	bool bShouldSuppressCursor;

	/** Whether or not to force the cursor on the viewport */
	bool bShouldForceCursor;

	/** The current tick number */
	uint32 CurrentTickNumber;

	/** Container of assets */
	UPROPERTY()
	TObjectPtr<const UViewportInteractionAssetContainer> AssetContainer;

	/** If we want to skip playing the sound when refreshing the transform gizmo next time */
	bool bPlayNextRefreshTransformGizmoSound;

	/** Slate Input Processor */
	TSharedPtr<class FViewportInteractionInputProcessor> InputProcessor;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
