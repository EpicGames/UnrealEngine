// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
    [SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class ViewportInteraction : ModuleRules
    {
        public ViewportInteraction(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "InputCore",
					"EditorFramework",
                    "UnrealEd",
                    "Slate",
                    "SlateCore",
					"RenderCore",
					"RHI"
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "LevelEditor"
                }
            );
        }
    }
}
